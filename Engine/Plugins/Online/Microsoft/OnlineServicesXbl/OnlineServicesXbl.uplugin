{
	"FileVersion": 3,

	"FriendlyName": "Online Services Xbox Live",
	"Version": 1,
	"VersionName": "1.0",
	"Description": "Online Services implementation for Xbox Live.",
	"Category": "Online Platform",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"EnabledByDefault": false,
	"HasExplicitPlatforms": true,	
	"SupportedTargetPlatforms": ["Win64"],
	"Modules": [
		{
			"Name": "OnlineServicesXbl",
			"Type": "RuntimeNoCommandlet",
			"LoadingPhase": "PostConfigInit",
			"HasExplicitPlatforms": true,
			"PlatformAllowList": [ "Win64" ]
		}
	],
	"Plugins": [
		{
			"Name": "OnlineServices",
			"Enabled": true
		}
	]
}