// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class OnlineServicesXbl : ModuleRules
{
	public OnlineServicesXbl(ReadOnlyTargetRules Target) : base(Target)
	{
		if (GRDK.IsValid(Target))
		{
			ExtraRootPath = ("GSDK", GRDK.GetGDKRoot());
		}

		PublicDependencyModuleNames.AddRange(
			[
				"Core",
				"CoreOnline",
				"Sockets",
				"OnlineServicesInterface",
				"OnlineServicesCommon",
				"GRDK",
				"GDKRuntime",
			]
		);

		PrivateDependencyModuleNames.AddRange(
			[ 
				"ApplicationCore",
				"Engine",
				"OnlineBase",
				"OnlineServicesCommonEngineUtils",
				"XSAPI",
				"Json",
			]
		);
		RuntimeDependencies.Add("$(ProjectDir)/Config/Xbl/*", StagedFileType.NonUFS);
		RuntimeDependencies.Add("$(ProjectDir)/Platforms/GDK/Config/OSS/*", StagedFileType.NonUFS); // legacy path
	}
}
