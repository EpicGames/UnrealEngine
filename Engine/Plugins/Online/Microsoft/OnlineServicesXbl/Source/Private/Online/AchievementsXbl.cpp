// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK

#include "Online/AchievementsXbl.h"
#include "Online/AuthXbl.h"
#include "Online/OnlineUtils.h"
#include "Online/OnlineUtilsCommon.h"
#include "GDKRuntimeModule.h"
#include "Online/Windows/WindowsOnlineErrorDefinitions.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"

THIRD_PARTY_INCLUDES_START
#include <xsapi-c/achievements_c.h>
THIRD_PARTY_INCLUDES_END


namespace UE::Online {

	void FAchievementsXbl::Initialize()
	{
		FAchievementsCommon::Initialize();
		TOnlineComponent::LoadConfig(Config);		
		AchievementUpdateHandle = static_cast<FOnlineServicesXbl&>(Services).ContextManager->OnAchievementUpdate().Add(this, &FAchievementsXbl::AchievementUnlockNotificationHandler);
	}

	void FAchievementsXbl::PreShutdown()
	{
		AchievementUpdateHandle.Unbind();
		FAchievementsCommon::PreShutdown();		
	}

	void LexFromString(EXblAchievementMode& OutMode, const TCHAR* InStr)
	{
		OutMode = (FCString::Stricmp(InStr, TEXT("Mode2017")) == 0) ? EXblAchievementMode::Mode2017 : EXblAchievementMode::Default;
	}

	void FAchievementsXbl::AchievementUnlockNotificationHandler(const FAchievementUpdate& Update)
	{
		FAccountId AccountID = FOnlineAccountIdRegistryXbl::Get().Find(Update.XUID);
		if (!AccountID.IsValid())
		{
			return;
		}	
		FAchievementStateMap* UserAchievementStates = AchievementStates.Find(AccountID);
		if (!UserAchievementStates)
		{
			return;
		}
		TArray<FString> UpdatedIDs;
		for (const FAchievementState& State : Update.AchievementUpdates)
		{
			FString AchievementName = GetAchievementNameFromId(FCString::Atoi(*State.AchievementId));
			if(UserAchievementStates->Contains(AchievementName) &&
				!FMath::IsNearlyEqual((*UserAchievementStates)[AchievementName].Progress, State.Progress, UE_KINDA_SMALL_NUMBER))
			{
				(*UserAchievementStates)[AchievementName] = State;
				UpdatedIDs.Emplace(AchievementName);
			}
		}
		OnAchievementStateUpdatedEvent.Broadcast({ AccountID ,MoveTemp(UpdatedIDs) });
	}

	TOnlineAsyncOpHandle<FUnlockAchievements> FAchievementsXbl::UnlockAchievements(FUnlockAchievements::Params&& Params)
	{
		TOnlineAsyncOpRef<FUnlockAchievements> Op = GetOp<FUnlockAchievements>(MoveTemp(Params));

		if (!Services.Get<FAuthXbl>()->IsLoggedIn(Op->GetParams().LocalAccountId))
		{
			Op->SetError(Errors::InvalidUser());
			return Op->GetHandle();
		}
		if (Op->GetParams().AchievementIds.IsEmpty())
		{
			Op->SetError(Errors::InvalidParams());
			return Op->GetHandle();
		}
		if (!AchievementDefinitions.IsSet())
		{
			// Call QueryAchievement... first
			Op->SetError(Errors::InvalidState());
			return Op->GetHandle();
		}
		if (!Op->IsReady())
		{
			if (Config.AchivementMode == EXblAchievementMode::Mode2017)
			{
				Op->Then([this](TOnlineAsyncOp<FUnlockAchievements>& InAsyncOp)
					{
						const FUnlockAchievements::Params& Params = InAsyncOp.GetParams();
						FAchievementStateMap* LocalUserAchievementStates = AchievementStates.Find(Params.LocalAccountId);
						FGDKContextHandle GDKContext = static_cast<FOnlineServicesXbl&>(Services).ContextManager->GetGDKContext(FOnlineAccountIdRegistryXbl::Get().Find(Params.LocalAccountId));
						if(!GDKContext.IsValid())
						{
							InAsyncOp.SetError(Errors::InvalidUser());
							return;

						}
						int Unlocks = 0;
						uint64 XUID = FOnlineAccountIdRegistryXbl::Get().Find(Params.LocalAccountId);
						for (const FString& AchievementId : Params.AchievementIds)
						{
							const FAchievementState* AchievementState = LocalUserAchievementStates->Find(AchievementId);
							if (!AchievementState)
							{
								InAsyncOp.SetError(Errors::NotFound());
								UE_LOGF(LogOnlineServices, Verbose, "[%s]: No state found attempting to unlock achievement [%ls]", __FUNCTION__, *AchievementId);
								continue;
							}

							uint32 Percent = AchievementState->Progress * 100;
							Percent = FMath::Clamp(Percent, 0, 100);
							HRESULT Result = AsyncGDKTask([GDKContext, XUID, Percent, AchievementId](XAsyncBlock* AsyncBlock)
								{ 
									return XblAchievementsUpdateAchievementAsync(GDKContext, XUID, TCHAR_TO_UTF8(*AchievementId), Percent, AsyncBlock);
								});

							if(FAILED(Result))
							{
								UE_LOGF(LogOnlineServices, Verbose, "[%s]: Error [%ls] from request to update achievement [%ls]", __FUNCTION__,
									*Errors::FromHRESULT(Result).GetLogString(), *AchievementId);

							}			
							Unlocks++;
						}
						if (Unlocks == 0)
						{
							UE_LOGF(LogOnlineServices, Warning, "[%s]: No ahievements updated", __FUNCTION__);
							InAsyncOp.SetError(Errors::InvalidParams());
						}
						else
						{
							InAsyncOp.SetResult({});
						}
						return;

					}, FOnlineAsyncExecutionPolicy::RunOnThreadPool()).Enqueue(Services.GetParallelQueue());
			}
			else
			{
				Op->Then([this](TOnlineAsyncOp<FUnlockAchievements>& InAsyncOp)
					{
						const FUnlockAchievements::Params& Params = InAsyncOp.GetParams();
						FAchievementStateMap* LocalUserAchievementStates = AchievementStates.Find(Params.LocalAccountId);
						int Unlocks = 0;
						for (const FString& AchievementName : Params.AchievementIds)
						{
							const FAchievementState* AchievementState = LocalUserAchievementStates->Find(AchievementName);
							if (!AchievementState)
							{
								InAsyncOp.SetError(Errors::NotFound());
								UE_LOGF(LogOnlineServices, Verbose, "[%s]: No state found attempting to unlock achievement [%ls]", __FUNCTION__, *AchievementName);
								continue;
							}
							if (FMath::IsNearlyEqual(AchievementState->Progress,1.0f,UE_KINDA_SMALL_NUMBER))
							{
								UE_LOGF(LogOnlineServices, Verbose, "[%s]: Attempting to unlock achievement [%ls] already unlocked", __FUNCTION__, *AchievementName);
								return;
							}
							int64 AchievementId = GetAchievementIdFromName(AchievementName);
							

							FOnlineEventParams EventParams;
							EventParams.Add(TEXT("AchievementIndex"), FSchemaVariant(AchievementId));

							if (!static_cast<FOnlineServicesXbl&>(Services).EventLauncher->TriggerEvent(Params.LocalAccountId, *GetAchievementEventName(), EventParams))
							{
								continue;
							}
							Unlocks++;
						}
						if (Unlocks == 0)
						{
							UE_LOGF(LogOnlineServices, Warning, "[%s]: No valid ahievements to unlock, requested are either unknown or already unlocked", __FUNCTION__);
							InAsyncOp.SetError(Errors::InvalidParams());
						}
						else
						{
							InAsyncOp.SetResult({});
						}

					}, FOnlineAsyncExecutionPolicy::RunOnThreadPool()).Enqueue(Services.GetParallelQueue());
			}			
		}
		return Op->GetHandle();
	}


	TOnlineAsyncOpHandle<FQueryAchievementDefinitions> FAchievementsXbl::QueryAchievementDefinitions(FQueryAchievementDefinitions::Params&& InParams)
	{
		TOnlineAsyncOpRef<FQueryAchievementDefinitions> Op = GetJoinableOp<FQueryAchievementDefinitions>(MoveTemp(InParams));
		if (Op->IsReady())
		{
			return Op->GetHandle();
		}

		Op->Then([this](TOnlineAsyncOp<FQueryAchievementDefinitions>& Op)
			{
				const FQueryAchievementDefinitions::Params& Params = Op.GetParams();

				TSharedRef<TPromise<void>> Promise = MakeShared<TPromise<void>>();
				TFuture<void> Future = Promise->GetFuture();

				// Getting achievement states and details are both done with the same platform service request.
				// So we just have this details query run a state query internally. 
				FQueryAchievementStates::Params StateParams;
				StateParams.LocalAccountId = Params.LocalAccountId;
				TOnlineAsyncOpHandle<FQueryAchievementStates> QueryHandle = QueryAchievementStates(MoveTemp(StateParams));
				QueryHandle.OnComplete([WeakOp = Op.AsWeak(),Promise](const TOnlineResult<FQueryAchievementStates>& Result)
					{
						if (TOnlineAsyncOpPtr<FQueryAchievementDefinitions> StrongOp = WeakOp.Pin())
						{
							if(Result.IsError())
							{
								FOnlineError Error = Result.GetErrorValue();
								StrongOp->SetError(MoveTemp(Error));
							}
							else
							{
								StrongOp->SetResult({});
							}
						}
						Promise->EmplaceValue();
					});
				return Future;
			})
			.Enqueue(Services.GetParallelQueue());

		return Op->GetHandle();
	}

	TOnlineAsyncOpHandle<FQueryAchievementStates> FAchievementsXbl::QueryAchievementStates(FQueryAchievementStates::Params&& InParams)
	{
		TOnlineAsyncOpRef<FQueryAchievementStates> Op = GetJoinableOp<FQueryAchievementStates>(MoveTemp(InParams));
		if (Op->IsReady())
		{
			return Op->GetHandle();
		}
		Op->Then([this](TOnlineAsyncOp<FQueryAchievementStates>& Op)
			{
				TSharedRef<TPromise<void>> Promise = MakeShared<TPromise<void>>();
				TFuture<void> Future = Promise->GetFuture();
				TSharedRef<FGDKAsyncBlock> AsyncBlock = MakeShared<FGDKAsyncBlock>(nullptr, [Promise](class FGDKAsyncBlock*)
					{
						//process results in next step
						Promise->EmplaceValue();
					});
				Op.Data.Set<TSharedRef<FGDKAsyncBlock>>(UE_XBL_ASYNC_BLOCK_KEY_NAME, AsyncBlock);

				const FQueryAchievementStates::Params& Params = Op.GetParams();
				if (!Params.LocalAccountId.IsValid())
				{
					Op.SetError(Errors::InvalidParams());
					Promise->EmplaceValue();
					return Future;
				}

				if (!Services.Get<FAuthXbl>()->IsLoggedIn(Params.LocalAccountId))
				{
					Op.SetError(Errors::NotLoggedIn());
					Promise->EmplaceValue();
					return Future;
				}

				FGDKContextHandle GDKContext = static_cast<FOnlineServicesXbl&>(Services).ContextManager->GetGDKContext(FOnlineAccountIdRegistryXbl::Get().Find(Params.LocalAccountId));
				uint64 XUID = FOnlineAccountIdRegistryXbl::Get().Find(Params.LocalAccountId);

				HRESULT Result = XblAchievementsGetAchievementsForTitleIdAsync(
					GDKContext, XUID, IGDKRuntimeModule::Get().GetTitleId(), XblAchievementType::All,
					false, XblAchievementOrderBy::DefaultOrder, 0, 0, *AsyncBlock);

				if (FAILED(Result))
				{
					FOnlineError Error = Errors::FromHRESULT(Result);
					Op.SetError(MoveTemp(Error));
					Promise->EmplaceValue();
					UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to query achievements. Error %ls", __FUNCTION__, *Error.GetLogString());
					return Future;
				}
				return Future;
			})
			.Then([this](TOnlineAsyncOp<FQueryAchievementStates>& Op)
				{
					const TSharedRef<FGDKAsyncBlock>& AsyncBlock = GetOpDataChecked<TSharedRef<FGDKAsyncBlock>>(Op, UE_XBL_ASYNC_BLOCK_KEY_NAME);
					XblAchievementsResultHandle ResultHandle = nullptr;
					HRESULT Result = XblAchievementsGetAchievementsForTitleIdResult(*AsyncBlock, &ResultHandle);
					if (FAILED(Result))
					{
						FOnlineError Error = Errors::FromHRESULT(Result);
						UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed query achievements. Error %ls", __FUNCTION__, *Error.GetLogString());
						Op.SetError(MoveTemp(Error));
						return;
					}
					const XblAchievement* Achievements = nullptr;
					size_t ItemCount = 0;
					Result = XblAchievementsResultGetAchievements(ResultHandle, &Achievements, &ItemCount);
					if (Result != S_OK)
					{
						XblAchievementsResultCloseHandle(ResultHandle);
						FOnlineError Error = Errors::FromHRESULT(Result);
						UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed get query achievement result. Error %ls", __FUNCTION__, *Error.GetLogString());
						Op.SetError(MoveTemp(Error));
						return;
					}

					const FQueryAchievementStates::Params& Params = Op.GetParams();
					FAchievementStateMap& StateMap = AchievementStates.FindOrAdd(Params.LocalAccountId);
					FAchievementDefinitionMap NewAchievementDefinitions;
					for (int32 Index = 0; Index < ItemCount; ++Index)
					{
						FString AchievementId = GetAchievementNameFromId(FCString::Atoi(UTF8_TO_TCHAR(Achievements[Index].id)));
						float Progress = 0;
						FDateTime UnlockTime;
						if (Achievements[Index].progressState == XblAchievementProgressState::Achieved)
						{
							Progress = 1.0f;
							UnlockTime = FDateTime::FromUnixTimestamp(Achievements[Index].progression.timeUnlocked);
						}

						StateMap.Add(AchievementId, { AchievementId , Progress,	UnlockTime });

						FAchievementDefinition& AchievementDefinition = NewAchievementDefinitions.Emplace(AchievementId);
						AchievementDefinition.AchievementId = MoveTemp(AchievementId);
						AchievementDefinition.UnlockedDisplayName = FText::FromString(UTF8_TO_TCHAR(Achievements[Index].name));
						AchievementDefinition.UnlockedDescription = FText::FromString(UTF8_TO_TCHAR(Achievements[Index].unlockedDescription));
						AchievementDefinition.LockedDisplayName = FText::FromString(UTF8_TO_TCHAR(Achievements[Index].name));
						AchievementDefinition.LockedDescription = FText::FromString(UTF8_TO_TCHAR(Achievements[Index].lockedDescription));

						AchievementDefinition.bIsHidden = Achievements[Index].isSecret;

						for(int i = 0;i< Achievements[Index].mediaAssetsCount;++i)
						{
							if(Achievements[Index].mediaAssets[i].mediaAssetType == XblAchievementMediaAssetType::Icon)
							{
								AchievementDefinition.UnlockedIconUrl = UTF8_TO_TCHAR(Achievements[Index].mediaAssets[i].url);
								AchievementDefinition.LockedIconUrl = UTF8_TO_TCHAR(Achievements[Index].mediaAssets[i].url);
								break;
							}
						}

					}
					AchievementDefinitions.Emplace(MoveTemp(NewAchievementDefinitions));

					XblAchievementsResultCloseHandle(ResultHandle);
					Op.SetResult({});
					return;
				});
			Op->Enqueue(Services.GetParallelQueue());

		return Op->GetHandle();
	}

	const FString& FAchievementsXbl::GetAchievementNameFromId(int32 AchievementId)
	{
		if (AchievementsConfig.AchievementMap.Num() == 0)
		{
			LoadAndInitFromJsonConfig(TEXT("Achievements.json"));
		}

		static const FString NoAchievment = FString("No Matching achievement");
		const FString* AchievementName = AchievementsConfig.AchievementMap.FindKey(AchievementId);
		return AchievementName ? *AchievementName : NoAchievment;
	}

	int32 FAchievementsXbl::GetAchievementIdFromName(const FString& AchievementName)
	{
		if (AchievementsConfig.AchievementMap.Num() == 0)
		{
			LoadAndInitFromJsonConfig(TEXT("Achievements.json"));
		}

		int32* AchievementId = AchievementsConfig.AchievementMap.Find(AchievementName);
		return AchievementId ? *AchievementId : -1;
	}

	const FString& FAchievementsXbl::GetAchievementEventName()
	{
		if (AchievementsConfig.AchievementMap.Num() == 0)
		{
			LoadAndInitFromJsonConfig(TEXT("Achievements.json"));
		}
		return AchievementsConfig.AchievementEventName;
	}

	bool FAchievementsXbl::LoadAndInitFromJsonConfig(const TCHAR* JsonConfigName)
	{
		// check legacy path first
		FString JsonConfigFilename = FPaths::ProjectDir() / TEXT("Platforms/GDK/Config/OSS") / JsonConfigName;
		if (!FPaths::FileExists(JsonConfigFilename))
		{
			JsonConfigFilename = FPaths::ProjectDir() / TEXT("Config/Xbl") / JsonConfigName;
		}
		else if (ensureMsgf(!FPaths::FileExists(FPaths::ProjectDir() / TEXT("Config/Xbl") / JsonConfigName), TEXT("%s file found in both deprecated Platforms/GDK/Config/OSS and new Config/Xbl paths - remove the deprecated one"), JsonConfigName))
		{
			UE_LOGF(LogOnlineServices, Warning, "[%s]: Platforms/GDK/Config/OSS/ path for %ls is deprecated - move it to Config/Xbl/", __FUNCTION__, JsonConfigName);
		}

		FString JsonText;

		if (!FFileHelper::LoadFileToString(JsonText, *JsonConfigFilename))
		{
			UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to find json OnlineService achievements config: %ls", __FUNCTION__, *JsonConfigFilename);
			return false;
		}

		if (!AchievementsConfig.FromJson(JsonText))
		{
			UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to parse json OSS achievements config: %ls", __FUNCTION__, *JsonConfigFilename);
			return false;
		}

		return true;
	}

	TOnlineResult<FGetAchievementIds> FAchievementsXbl::GetAchievementIds(FGetAchievementIds::Params&& Params)
	{
		if (!Services.Get<FAuthXbl>()->IsLoggedIn(Params.LocalAccountId))
		{
			return TOnlineResult<FGetAchievementIds>(Errors::InvalidUser());
		}

		if (!AchievementDefinitions.IsSet())
		{
			// Call QueryAchievementDefinitions first
			return TOnlineResult<FGetAchievementIds>(Errors::InvalidState());
		}

		FGetAchievementIds::Result Result;
		AchievementDefinitions->GenerateKeyArray(Result.AchievementIds);
		return TOnlineResult<FGetAchievementIds>(MoveTemp(Result));
	}

	TOnlineResult<FGetAchievementDefinition> FAchievementsXbl::GetAchievementDefinition(FGetAchievementDefinition::Params&& Params)
	{
		if (!Services.Get<FAuthXbl>()->IsLoggedIn(Params.LocalAccountId))
		{
			return TOnlineResult<FGetAchievementDefinition>(Errors::InvalidUser());
		}

		if (!AchievementDefinitions.IsSet())
		{
			// Should call QueryAchievementDefinitions first
			return TOnlineResult<FGetAchievementDefinition>(Errors::InvalidState());
		}

		const FAchievementDefinition* AchievementDefinition = AchievementDefinitions->Find(Params.AchievementId);
		if (!AchievementDefinition)
		{
			return TOnlineResult<FGetAchievementDefinition>(Errors::NotFound());
		}

		return TOnlineResult<FGetAchievementDefinition>({ *AchievementDefinition });
	}

	TOnlineResult<FGetAchievementState> FAchievementsXbl::GetAchievementState(FGetAchievementState::Params&& Params) const
	{
		if (!Services.Get<FAuthXbl>()->IsLoggedIn(Params.LocalAccountId))
		{
			return TOnlineResult<FGetAchievementState>(Errors::InvalidUser());
		}

		const FAchievementStateMap* LocalUserAchievementStates = AchievementStates.Find(Params.LocalAccountId);
		if (!LocalUserAchievementStates)
		{
			// Call QueryAchievementStates first
			return TOnlineResult<FGetAchievementState>(Errors::InvalidState());
		}

		const FAchievementState* AchievementState = LocalUserAchievementStates->Find(Params.AchievementId);
		if (!AchievementState)
		{
			return TOnlineResult<FGetAchievementState>(Errors::NotFound());
		}

		return TOnlineResult<FGetAchievementState>({ *AchievementState });
	}


/* UE::Online */ }

#endif // WITH_GRDK

