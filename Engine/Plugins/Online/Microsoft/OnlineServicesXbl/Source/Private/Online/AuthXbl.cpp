// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK

#include "Online/AuthXbl.h"

#include "Algo/Find.h"
#include "Algo/ForEach.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/ScopeRWLock.h"
#include "Online/OnlineServicesXbl.h"
#include "Online/OnlineUtils.h"
#include "Online/OnlineUtilsCommon.h"
#include "Online/Windows/WindowsOnlineErrorDefinitions.h"
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"

namespace UE::Online {

const uint8 OnlineIdXblBufferLength = sizeof(uint64);

struct FAuthXblRelyingPartyDefinition
{
	FString RelyingParty;
	FString ServerURL;
};

struct FAuthXblQueryExternalAuthTokenConfig
{
	TArray<FAuthXblRelyingPartyDefinition> RelyingPartyDefinitions;
};

namespace Meta {

BEGIN_ONLINE_STRUCT_META(FAuthXblRelyingPartyDefinition)
	ONLINE_STRUCT_FIELD(FAuthXblRelyingPartyDefinition, RelyingParty),
	ONLINE_STRUCT_FIELD(FAuthXblRelyingPartyDefinition, ServerURL)
END_ONLINE_STRUCT_META()

BEGIN_ONLINE_STRUCT_META(FAuthXblQueryExternalAuthTokenConfig)
	ONLINE_STRUCT_FIELD(FAuthXblQueryExternalAuthTokenConfig, RelyingPartyDefinitions)
END_ONLINE_STRUCT_META()

/* Meta */ }

TSharedPtr<FAccountInfoXbl> FAccountInfoRegistryXbl::FindOrCreate(FPlatformUserId PlatformUserId, FOnlineServicesXbl& Service)
{
	{
		FReadScopeLock ReadLock(IndexLock);
		if (TSharedPtr<FAccountInfoXbl> FoundAccountInfo = StaticCastSharedPtr<FAccountInfoXbl>(FindNoLock(PlatformUserId)))
		{
			return FoundAccountInfo.ToSharedRef();
		}
	}

	{
		FWriteScopeLock WriteLock(IndexLock);
		if (TSharedPtr<FAccountInfoXbl> FoundAccountInfo = StaticCastSharedPtr<FAccountInfoXbl>(FindNoLock(PlatformUserId)))
		{
			return FoundAccountInfo.ToSharedRef();
		}

		uint64 XUID = 0;
		FGDKUserHandle UserHandle = IGDKRuntimeModule::Get().GetUserHandleByPlatformId(PlatformUserId);
		if( !UserHandle.IsValid())
		{
			// This can happen when launching with no accounts on the console. Which should only be possible using the dev tools.
			UE_LOGF(LogOnlineServices, Warning, "[%s] No user for platform ID %d", __FUNCTION__, PlatformUserId.GetInternalId());
			return nullptr;
		}
		verify(SUCCEEDED(XUserGetId(UserHandle, &XUID)));

		TSharedRef<FAccountInfoXbl> NewAccountInfo = MakeShared<FAccountInfoXbl>();
		NewAccountInfo->AccountId = FOnlineAccountIdRegistryXbl::Get().FindOrAddAccountId(XUID);
		NewAccountInfo->PlatformUserId = PlatformUserId;
		NewAccountInfo->LoginStatus = ELoginStatus::LoggedIn;
		NewAccountInfo->UserHandle = UserHandle;
		DoRegister(NewAccountInfo);
		Service.ContextManager->CreateGDKContext(UserHandle);
		return NewAccountInfo;
	}
}

TSharedPtr<FAccountInfoXbl> FAccountInfoRegistryXbl::Find(FPlatformUserId PlatformUserId) const
{
	return StaticCastSharedPtr<FAccountInfoXbl>(Super::Find(PlatformUserId));
}

TSharedPtr<FAccountInfoXbl> FAccountInfoRegistryXbl::Find(FAccountId AccountId) const
{
	return StaticCastSharedPtr<FAccountInfoXbl>(Super::Find(AccountId));
}

void FAccountInfoRegistryXbl::Register(const TSharedRef<FAccountInfoXbl>& AccountInfo)
{
	FWriteScopeLock Lock(IndexLock);
	DoRegister(AccountInfo);
}

void FAccountInfoRegistryXbl::Unregister(FAccountId AccountId)
{
	if (TSharedPtr<FAccountInfoXbl> AccountInfo = Find(AccountId))
	{
		FWriteScopeLock Lock(IndexLock);
		DoUnregister(AccountInfo.ToSharedRef());
	}
	else
	{
		UE_LOGF(LogOnlineServices, Warning, "[%s] Failed to find account [%ls].", __FUNCTION__, *ToLogString(AccountId));
	}
}

FAuthXbl::FAuthXbl(FOnlineServicesXbl& InServices)
	: FAuthCommon(InServices)
{
}

void FAuthXbl::Initialize()
{
	FAuthCommon::Initialize();
	InitializeUsers();
}

void FAuthXbl::PreShutdown()
{
	FAuthCommon::PreShutdown();
	UninitializeUsers();
}

TOnlineAsyncOpHandle<FAuthLogin> FAuthXbl::Login(FAuthLogin::Params&& Params)
{
	TOnlineAsyncOpRef<FAuthLogin> Operation = GetOp<FAuthLogin>(MoveTemp(Params));

	FAuthGetLocalOnlineUserByPlatformUserId::Params AuthParams;
	AuthParams.PlatformUserId = Params.PlatformUserId;
	TOnlineResult<FAuthGetLocalOnlineUserByPlatformUserId> AuthResult = GetLocalOnlineUserByPlatformUserId(MoveTemp(AuthParams));
	if (AuthResult.IsOk())
	{
		Operation->SetResult(FAuthLogin::Result{ MoveTemp(AuthResult.GetOkValue().AccountInfo) });
	}
	else
	{
		Operation->SetError(Errors::InvalidParams());
	}

	return Operation->GetHandle();
}

TOnlineAsyncOpHandle<FAuthQueryExternalAuthToken> FAuthXbl::QueryExternalAuthToken(FAuthQueryExternalAuthToken::Params&& Params)
{
	TOnlineAsyncOpRef<FAuthQueryExternalAuthToken> Op = GetJoinableOp<FAuthQueryExternalAuthToken>(MoveTemp(Params));
	if (!Op->IsReady())
	{
		Op->Then([this](TOnlineAsyncOp<FAuthQueryExternalAuthToken>& InAsyncOp)
		{
			const FAuthQueryExternalAuthToken::Params& Params = InAsyncOp.GetParams();

			TSharedPtr<FAccountInfoXbl> AccountInfo = AccountInfoRegistry.Find(Params.LocalAccountId);
			if (!AccountInfo)
			{
				InAsyncOp.SetError(Errors::InvalidParams());
				return;
			}

			// Store account info on operation.
			InAsyncOp.Data.Set<TSharedRef<FAccountInfoXbl>>(UE_XBL_ACCOUNT_INFO_KEY_NAME, AccountInfo.ToSharedRef());

			// The primary external auth method is the XSTS token configured for the SSO XSTS URL.
			if (Params.Method != EExternalAuthTokenMethod::Primary)
			{
				InAsyncOp.SetError(Errors::InvalidParams());
				return;
			}
		})
		.Then([this](TOnlineAsyncOp<FAuthQueryExternalAuthToken>& InAsyncOp)
		{
			const FAuthQueryExternalAuthToken::Params& Params = InAsyncOp.GetParams();
			const TSharedRef<FAccountInfoXbl>& AccountInfo = GetOpDataChecked<TSharedRef<FAccountInfoXbl>>(InAsyncOp, UE_XBL_ACCOUNT_INFO_KEY_NAME);

			TSharedRef<TPromise<void>> Promise = MakeShared<TPromise<void>>();
			TFuture<void> Future = Promise->GetFuture();

			TSharedRef<FGDKAsyncBlock> AsyncBlock = MakeShared<FGDKAsyncBlock>(nullptr, [Promise](class FGDKAsyncBlock*)
			{
				Promise->EmplaceValue();
			});

			// Capture async block on operation.
			InAsyncOp.Data.Set<TSharedRef<FGDKAsyncBlock>>(UE_XBL_ASYNC_BLOCK_KEY_NAME, AsyncBlock);

			FAuthXblQueryExternalAuthTokenConfig QueryExternalAuthTokenConfig;
			LoadConfig(QueryExternalAuthTokenConfig, TEXT("QueryExternalAuthToken"));

			FAuthXblRelyingPartyDefinition* RelyingPartyDefinition = Algo::FindByPredicate(QueryExternalAuthTokenConfig.RelyingPartyDefinitions, [&Params](const FAuthXblRelyingPartyDefinition& Definition)->bool { return Definition.RelyingParty == Params.RelyingParty; });
			if (RelyingPartyDefinition == nullptr)
			{
				UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to find config for relying party %ls", __FUNCTION__, *Params.RelyingParty);
				InAsyncOp.SetError(Errors::NotConfigured());
				Promise->EmplaceValue();
				return Future;
			}

			UE_LOGF(LogOnlineServices, VeryVerbose, "[%s]: Using relying party %ls", __FUNCTION__, *Params.RelyingParty);

			HRESULT Result = XUserGetTokenAndSignatureUtf16Async(
				AccountInfo->UserHandle,
				XUserGetTokenAndSignatureOptions::None,
				TEXT("GET"),
				*RelyingPartyDefinition->ServerURL,
				0,
				nullptr,
				0,
				nullptr,
				*AsyncBlock);
			if (Result != S_OK)
			{
				FOnlineError Error = Errors::FromHRESULT(Result);
				UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to get token: %ls", __FUNCTION__, *Error.GetLogString());
				InAsyncOp.SetError(MoveTemp(Error));
				Promise->EmplaceValue();
				return Future;
			}

			return Future;
		})
		.Then([this](TOnlineAsyncOp<FAuthQueryExternalAuthToken>& InAsyncOp)
		{
			const TSharedRef<FGDKAsyncBlock>& AsyncBlock = GetOpDataChecked<TSharedRef<FGDKAsyncBlock>>(InAsyncOp, UE_XBL_ASYNC_BLOCK_KEY_NAME);

			uint64 ResultSize = 0;
			HRESULT Result = XUserGetTokenAndSignatureUtf16ResultSize(*AsyncBlock, &ResultSize);
			if (Result != S_OK)
			{
				FOnlineError Error = Errors::FromHRESULT(Result);
				UE_LOGF(LogOnlineServices, Warning, "[%s]: GetTokenAndSignatureResultSize failed: %ls", __FUNCTION__, *Error.GetLogString());
				InAsyncOp.SetError(MoveTemp(Error));
				return;
			}

			TArray<uint8> BufferArray;
			BufferArray.Reserve(ResultSize);
			XUserGetTokenAndSignatureUtf16Data* ResultData = nullptr;

			Result = XUserGetTokenAndSignatureUtf16Result(
				*AsyncBlock,
				ResultSize,
				BufferArray.GetData(),
				&ResultData,
				nullptr);
			if (Result != S_OK)
			{
				FOnlineError Error = Errors::FromHRESULT(Result);
				UE_LOGF(LogOnlineServices, Warning, "[%s]: GetTokenAndSignatureResult failed: %ls", __FUNCTION__, *Error.GetLogString());
				InAsyncOp.SetError(MoveTemp(Error));
				return;
			}

			FExternalAuthToken ExternalAuthToken;
			ExternalAuthToken.Type = ExternalLoginType::XblXstsToken;
			ExternalAuthToken.Data = ResultData->token;
			InAsyncOp.SetResult(FAuthQueryExternalAuthToken::Result{ MoveTemp(ExternalAuthToken) });
		})
		.Enqueue(GetSerialQueue());
	}

	return Op->GetHandle();
}

const FAccountInfoRegistry& FAuthXbl::GetAccountInfoRegistry() const
{
	return AccountInfoRegistry;
}

void FAuthXbl::InitializeUsers()
{
	// There is no "login" for Xbox - all local users are initialized as "logged in".
	TArray<FPlatformUserId> Users;
	IPlatformInputDeviceMapper::Get().GetAllActiveUsers(Users);
	Algo::ForEach(Users, [&](FPlatformUserId PlatformUserId)
	{
		AccountInfoRegistry.FindOrCreate(PlatformUserId, static_cast<FOnlineServicesXbl&>(Services));
	});

	// Setup hook to add new users when they become available.
	IPlatformInputDeviceMapper::Get().GetOnInputDeviceConnectionChange().AddRaw(this, &FAuthXbl::OnInputDeviceConnectionChange);
}

void FAuthXbl::UninitializeUsers()
{
	IPlatformInputDeviceMapper::Get().GetOnInputDeviceConnectionChange().RemoveAll(this);
}

void FAuthXbl::OnInputDeviceConnectionChange(EInputDeviceConnectionState NewConnectionState, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId)
{
	// If this is a new platform user then register an entry for them so they will be seen as "logged-in".
	if (NewConnectionState == EInputDeviceConnectionState::Connected && PlatformUserId != PLATFORMUSERID_NONE && !AccountInfoRegistry.Find(PlatformUserId))
	{
		TSharedPtr<FAccountInfoXbl> AccountInfo = AccountInfoRegistry.FindOrCreate(PlatformUserId, static_cast<FOnlineServicesXbl&>(Services));
		if( AccountInfo.IsValid())
		{
			OnAuthLoginStatusChangedEvent.Broadcast(FAuthLoginStatusChanged{ AccountInfo.ToSharedRef(), ELoginStatus::LoggedIn });
		}
	}
}

// FOnlineAccountIdRegistryXbl
FOnlineAccountIdRegistryXbl& FOnlineAccountIdRegistryXbl::Get()
{
	static FOnlineAccountIdRegistryXbl Instance;
	return Instance;
}

FAccountId FOnlineAccountIdRegistryXbl::Find(const uint64 XUID) const
{
	const FReadScopeLock ReadLock(Lock);
	const FOnlineAccountIdXbl* Entry = FindNoLock(XUID);
	return Entry ? Entry->AccountId : FAccountId();
}

uint64 FOnlineAccountIdRegistryXbl::Find(const FAccountId& AccountId) const
{
	const FReadScopeLock ReadLock(Lock);
	const FOnlineAccountIdXbl* Entry = FindNoLock(AccountId);
	return Entry ? Entry->XUID : 0;
}

const FOnlineAccountIdXbl* FOnlineAccountIdRegistryXbl::FindNoLock(const FAccountId& AccountId) const
{
	if(AccountId.IsValid() && AccountId.GetOnlineServicesType() == EOnlineServices::Xbox && AccountId.GetHandle() <= (uint32)Ids.Num())
	{
		return &Ids[AccountId.GetHandle()-1];
	}
	return nullptr;
}

const FOnlineAccountIdXbl* FOnlineAccountIdRegistryXbl::FindNoLock(const uint64 XUID) const
{
	FOnlineAccountIdXbl* const* Entry = XUIDIndex.Find(XUID);
	return Entry ? *Entry : nullptr;
}

//CDATODO refactor this away? need to knowledge possibility of failure
FAccountId FOnlineAccountIdRegistryXbl::FindOrAddAccountId(const uint64 XUID)
{
	// Check for existing entry under read lock.
	{
		FReadScopeLock ReadLock(Lock);
		if (const FOnlineAccountIdXbl* ExistingAccountId = FindNoLock(XUID))
		{
			return ExistingAccountId->AccountId;
		}
	}

	// Check for existing entry again under write lock before adding entry.
	{
		FWriteScopeLock WriteLock(Lock);
		if (const FOnlineAccountIdXbl* ExistingAccountId = FindNoLock(XUID))
		{
			return ExistingAccountId->AccountId;
		}

		FOnlineAccountIdXbl& Id = Ids.Emplace_GetRef();
		Id.XUID = XUID;
		Id.AccountId = FAccountId(EOnlineServices::Xbox, Ids.Num());

		XUIDIndex.Add(XUID, &Id);
		return Id.AccountId;
	}
}

FString FOnlineAccountIdRegistryXbl::ToString(const FAccountId& AccountId) const
{
	const FReadScopeLock ReadLock(Lock);
	if(const FOnlineAccountIdXbl* Id = FindNoLock(AccountId))
	{
		return FString::Printf(TEXT("%llu"), Id->XUID);
	}


	return FString(TEXT("<invalid>"));
}

FString FOnlineAccountIdRegistryXbl::ToLogString(const FAccountId& AccountId) const
{
	return ToString(AccountId);
}

TArray<uint8> FOnlineAccountIdRegistryXbl::ToReplicationData(const FAccountId& AccountId) const
{
	const FReadScopeLock ReadLock(Lock);
	if (const FOnlineAccountIdXbl* Id = FindNoLock(AccountId))
	{
		TArray<uint8> ReplicationData;
		ReplicationData.SetNumUninitialized(OnlineIdXblBufferLength);
		*reinterpret_cast<uint64*>(ReplicationData.GetData()) = Id->XUID;
		return ReplicationData;
	}

	return TArray<uint8>();
}

FAccountId FOnlineAccountIdRegistryXbl::FromReplicationData(const TArray<uint8>& ReplicationData)
{
	if (ReplicationData.Num() == OnlineIdXblBufferLength)
	{
		uint64 XUID = *reinterpret_cast<const uint64*>(ReplicationData.GetData());
		return FindOrAddAccountId(XUID);
	}

	return FAccountId();
}

FAccountId FOnlineAccountIdRegistryXbl::FromStringData(const FString& StringData)
{
	if (!StringData.IsEmpty())
	{
		uint64 XUID = FCString::Strtoui64(*StringData, NULL, 10);
		return FindOrAddAccountId(XUID);
	}

	return FAccountId();
}

/* UE::Online */ }

#endif // WITH_GRDK
