// Copyright Epic Games, Inc. All Rights Reserved.
#if WITH_GRDK

#include "Online/CommerceXbl.h"
#include "Online/OnlineServicesXbl.h"
#include "Online/AuthXbl.h"
#include "Online/OnlineUtils.h"
#include "Online/OnlineUtilsCommon.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"

#include "Online/Windows/WindowsOnlineErrorDefinitions.h"

#include "Internationalization/Culture.h"
#include "Internationalization/FastDecimalFormat.h"



#define DEBUG_LOG_GDK_STORE_ITEMS 0

#define UE_XBL_PRODUCT_KEY_NAME  TEXT("ProductHandle")
#define UE_XBL_OFFERIDS_KEY_NAME  TEXT("OFFERIDs")
namespace UE::Online {


	XStoreProductKind AllProductKinds =
		XStoreProductKind::Consumable |
		XStoreProductKind::Durable |
		XStoreProductKind::Game |
		XStoreProductKind::Pass |
		XStoreProductKind::UnmanagedConsumable;


	FEntitlement XStoreProductToFEntitlement(const XStoreProduct* Product,uint64 XUID)
	{
		FEntitlement NewEntitlement;

#if DEBUG_LOG_GDK_STORE_ITEMS && !UE_BUILD_SHIPPING
		UE_LOGF(LogOnlineServices, Log, "ProductDetails: %ls = %ls", TEXT("storeId"), Product->storeId ? UTF8_TO_TCHAR(Product->storeId) : TEXT(""));
		UE_LOGF(LogOnlineServices, Log, "ProductDetails: %ls = %ls", TEXT("title"), Product->title ? UTF8_TO_TCHAR(Product->title) : TEXT(""));
		UE_LOGF(LogOnlineServices, Log, "ProductDetails: %ls = %ls", TEXT("description"), Product->description ? UTF8_TO_TCHAR(Product->description) : TEXT(""));
		UE_LOGF(LogOnlineServices, Log, "ProductDetails: %ls = %ls", TEXT("language"), Product->language ? UTF8_TO_TCHAR(Product->language) : TEXT(""));
		UE_LOGF(LogOnlineServices, Log, "ProductDetails: %ls = %ls", TEXT("inAppOfferToken"), Product->inAppOfferToken ? UTF8_TO_TCHAR(Product->inAppOfferToken) : TEXT(""));
		UE_LOGF(LogOnlineServices, Log, "ProductDetails: %ls = %ls", TEXT("linkUri"), Product->linkUri ? UTF8_TO_TCHAR(Product->linkUri) : TEXT(""));
		UE_LOGF(LogOnlineServices, Log, "ProductDetails: %ls = %ld", TEXT("productKind"), Product->productKind);
		UE_LOGF(LogOnlineServices, Log, "ProductDetails: %ls = %ls", TEXT("hasDigitalDownload"), *LexToString(Product->hasDigitalDownload));
		UE_LOGF(LogOnlineServices, Log, "ProductDetails: %ls = %ls", TEXT("isInUserCollection"), *LexToString(Product->isInUserCollection));
#endif
		NewEntitlement.ProductId = FString(UTF8_TO_TCHAR(Product->storeId));

		// Find the first SKU that is in user's collection.
		const XStoreCollectionData* CollectionData = nullptr;
		for (uint32_t SkuIndex = 0; SkuIndex < Product->skusCount; ++SkuIndex)
		{
			const XStoreSku& Sku = Product->skus[SkuIndex];
			if (Sku.isInUserCollection)
			{
				CollectionData = &Sku.collectionData;
				break;
			}
		}
		// We are querying the user collection, so there must be at least one SKU that's in user's collection.
		check(CollectionData);


		if (static_cast<uint32>(Product->productKind & XStoreProductKind::Pass) != 0)
		{
			// For passes, we include the XUID, productId, and the acquired date to make each purchase unique.
			NewEntitlement.EntitlementId = FString::Printf(TEXT("%lld:%s:%lld"), XUID, *NewEntitlement.ProductId, CollectionData->acquiredDate);
		}
		else
		{
			// For everything else we maintain our existing behavior.
			NewEntitlement.EntitlementId = FString::Printf(TEXT("%s:%d:%s"), *NewEntitlement.ProductId, CollectionData->quantity, *FGuid::NewGuid().ToString());
		}

		NewEntitlement.AcquiredDate = FDateTime::FromUnixTimestamp(CollectionData->acquiredDate);
		NewEntitlement.ExpiryDate = FDateTime::FromUnixTimestamp(CollectionData->endDate);
		NewEntitlement.Quantity = CollectionData->quantity;
		NewEntitlement.bRedeemed = false; // CDA this doesn't map

		return NewEntitlement;
	}

	FOffer XStoreProductToFOffer(const XStoreProduct* Product)
	{
		FOffer NewOffer;

#if DEBUG_LOG_GDK_STORE_ITEMS && !UE_BUILD_SHIPPING
		UE_LOGF(LogOnlineServices, Log, "ProductDetails: %ls = %ls", TEXT("storeId"), Product->storeId ? UTF8_TO_TCHAR(Product->storeId) : TEXT(""));
		UE_LOGF(LogOnlineServices, Log, "ProductDetails: %ls = %ls", TEXT("title"), Product->title ? UTF8_TO_TCHAR(Product->title) : TEXT(""));
		UE_LOGF(LogOnlineServices, Log, "ProductDetails: %ls = %ls", TEXT("description"), Product->description ? UTF8_TO_TCHAR(Product->description) : TEXT(""));
		UE_LOGF(LogOnlineServices, Log, "ProductDetails: %ls = %ls", TEXT("language"), Product->language ? UTF8_TO_TCHAR(Product->language) : TEXT(""));
		UE_LOGF(LogOnlineServices, Log, "ProductDetails: %ls = %ls", TEXT("inAppOfferToken"), Product->inAppOfferToken ? UTF8_TO_TCHAR(Product->inAppOfferToken) : TEXT(""));
		UE_LOGF(LogOnlineServices, Log, "ProductDetails: %ls = %ls", TEXT("linkUri"), Product->linkUri ? UTF8_TO_TCHAR(Product->linkUri) : TEXT(""));
		UE_LOGF(LogOnlineServices, Log, "ProductDetails: %ls = %ld", TEXT("productKind"), Product->productKind);
		UE_LOGF(LogOnlineServices, Log, "ProductDetails: %ls = %ls", TEXT("hasDigitalDownload"), *LexToString(Product->hasDigitalDownload));
		UE_LOGF(LogOnlineServices, Log, "ProductDetails: %ls = %ls", TEXT("isInUserCollection"), *LexToString(Product->isInUserCollection));
#endif	

		NewOffer.OfferId = FString(UTF8_TO_TCHAR(Product->storeId));

		NewOffer.Title = FText::FromString(FString(UTF8_TO_TCHAR(Product->title)));
		NewOffer.Description = FText::FromString(FString(UTF8_TO_TCHAR(Product->description)));
		NewOffer.LongDescription = NewOffer.Description;

		NewOffer.FormattedRegularPrice = FText::FromString(FString(UTF8_TO_TCHAR(Product->price.formattedBasePrice)));
		NewOffer.RegularPrice = Product->price.basePrice;

		NewOffer.FormattedPrice = FText::FromString(FString(UTF8_TO_TCHAR(Product->price.formattedPrice)));

		NewOffer.CurrencyCode = FString(FString(UTF8_TO_TCHAR(Product->price.currencyCode)));

		const FCulture& Culture = *FInternationalization::Get().GetCurrentCulture();

		const FDecimalNumberFormattingRules& FormattingRules = Culture.GetCurrencyFormattingRules(NewOffer.CurrencyCode);
		const FNumberFormattingOptions& FormattingOptions = FormattingRules.CultureDefaultFormattingOptions;

		const double ValueAsDouble = static_cast<double>(Product->price.price) * static_cast<double>(FMath::Pow(10.0f, FormattingOptions.MaximumFractionalDigits));
		NewOffer.Price = FMath::RoundToInt(ValueAsDouble);

		NewOffer.PriceDecimalPoint = FormattingOptions.MinimumFractionalDigits;

		if (Product->price.isOnSale)
		{
			NewOffer.ExpirationDate = FDateTime::FromUnixTimestamp(Product->price.saleEndDate);
		}
		return NewOffer;
	}


FCommerceXbl::FCommerceXbl(FOnlineServicesXbl& InServices)
	: Super(InServices)
{
}

void FCommerceXbl::Initialize()
{
	Super::Initialize();
	FCoreDelegates::ApplicationWillEnterBackgroundDelegate.AddRaw(this, &FCommerceXbl::HandleSuspend);
}

void FCommerceXbl::Shutdown()
{
	FCoreDelegates::ApplicationWillEnterBackgroundDelegate.RemoveAll(this);
	Super::Shutdown();
}

bool FCommerceXbl::ParseProducts(FAccountId LocalAccountId, XStoreProductQueryHandle ProductQueryHandle,  TArray<FOfferId>* OfferIds)
{
	TArray<FOffer> FoundOffers;
	HRESULT Result = XStoreEnumerateProductsQuery(ProductQueryHandle, &FoundOffers, [](const XStoreProduct* Product, void* Context)
		{
			TArray<FOffer>* FoundOffers = static_cast<TArray<FOffer>*>(Context);

			if (Product)
			{
				FoundOffers->Emplace(XStoreProductToFOffer(Product));
			}
			return true;
		});
	if(FAILED(Result))
	{
		return false;
	}
	FScopeLock ScopeLock(&CachedOffersLock);

	for (FOffer& Found : FoundOffers)
	{
		if (OfferIds)
		{
			OfferIds->Add(Found.OfferId);
		}
		CachedOffers.FindOrAdd(LocalAccountId).Emplace(Found.OfferId, Found);
	}

	return true;
}


TOnlineAsyncOpHandle<FCommerceQueryOffers> FCommerceXbl::QueryOffers(FCommerceQueryOffers::Params&& InParams)
{
	TOnlineAsyncOpRef<FCommerceQueryOffers> Op = GetJoinableOp<FCommerceQueryOffers>(MoveTemp(InParams));
	if (Op->IsReady())
	{
		return Op->GetHandle();
	}
	
	
	const FCommerceQueryOffers::Params& Params = Op->GetParams();
	if (!Params.LocalAccountId.IsValid())
	{
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}

	Op->Then([this](TOnlineAsyncOp<FCommerceQueryOffers>& Op)
		{
			TSharedRef<TPromise<void>> Promise = MakeShared<TPromise<void>>();
			TFuture<void> Future = Promise->GetFuture();
			const FCommerceQueryOffers::Params& Params = Op.GetParams();

			uint64 LocalXUID = FOnlineAccountIdRegistryXbl::Get().Find(Params.LocalAccountId);
			FGDKUserHandle GDKUserHandle = IGDKRuntimeModule::Get().GetUserHandleByXUserId(LocalXUID);

			XStoreContextHandle StoreContextHandle = GetStoreContextHandle(GDKUserHandle);				

			TSharedRef<FGDKAsyncBlock> AsyncBlock = MakeShared<FGDKAsyncBlock>(nullptr,
				[Promise, WeakOperation = TWeakPtr<TOnlineAsyncOp<FCommerceQueryOffers>>(Op.AsShared())](class FGDKAsyncBlock* Block)
				{
					TSharedPtr<TOnlineAsyncOp<FCommerceQueryOffers>> Op = WeakOperation.Pin();
					if (Op)
					{
						HRESULT Result = Block->GetStatus();
						if (FAILED(Result))
						{
							FOnlineError Error = Errors::FromHRESULT(Result);
							UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to query products. Error %ls", __FUNCTION__, *Error.GetLogString());
							Op->SetError(MoveTemp(Error));
						}
					}
					else
					{
						UE_LOGF(LogOnlineServices, Warning, "[%s]: Opertion removed before async return.", __FUNCTION__);
					}
					Promise->EmplaceValue();
				});
			Op.Data.Set<TSharedRef<FGDKAsyncBlock>>(UE_XBL_ASYNC_BLOCK_KEY_NAME, AsyncBlock);

			HRESULT Result = XStoreQueryAssociatedProductsAsync(StoreContextHandle, AllProductKinds, UINT32_MAX, *AsyncBlock);
			if (FAILED(Result))
			{
				FOnlineError Error = Errors::FromHRESULT(Result);
				UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to query products. Error %ls", __FUNCTION__, *Error.GetLogString());
				Promise->EmplaceValue();
				Op.SetError(MoveTemp(Error));
				return Future;
			}
			return Future;

		}, FOnlineAsyncExecutionPolicy::RunOnThreadPool()).Then([this](TOnlineAsyncOp<FCommerceQueryOffers>& Op)
			{
				const TSharedRef<FGDKAsyncBlock>& AsyncBlock = GetOpDataChecked<TSharedRef<FGDKAsyncBlock>>(Op, UE_XBL_ASYNC_BLOCK_KEY_NAME);

				XStoreProductQueryHandle ProductQueryHandle = nullptr;
				HRESULT Result = XStoreQueryAssociatedProductsResult(*AsyncBlock, &ProductQueryHandle);

				if (FAILED(Result))
				{
					FOnlineError Error = Errors::FromHRESULT(Result);
					UE_LOGF(LogOnlineServices, Warning, "[%s]: QueryProducts failed. Error %ls", __FUNCTION__, *Error.GetLogString());
					Op.SetError(MoveTemp(Error));
					return;
				}

				const FCommerceQueryOffers::Params& Params = Op.GetParams();


				if (!ParseProducts(Params.LocalAccountId, ProductQueryHandle, nullptr))
				{
					XStoreCloseProductsQueryHandle(ProductQueryHandle);
					FOnlineError Error = Errors::FromHRESULT(Result);
					UE_LOGF(LogOnlineServices, Warning, "[%s]: QueryProducts failed to enumerate products. Error %ls", __FUNCTION__, *Error.GetLogString());
					Op.SetError(MoveTemp(Error));
					return;
				}		

				if (!XStoreProductsQueryHasMorePages(ProductQueryHandle))
				{
					XStoreCloseProductsQueryHandle(ProductQueryHandle);
					Op.SetResult({ });
				}
				else
				{
					Op.Data.Set<XStoreProductQueryHandle>(UE_XBL_PRODUCT_KEY_NAME, ProductQueryHandle);
				}

			}).Then([this](TOnlineAsyncOp<FCommerceQueryOffers>& Op) mutable
			{
					XStoreProductQueryHandle ProductQueryHandle = GetOpDataChecked<XStoreProductQueryHandle>(Op, UE_XBL_PRODUCT_KEY_NAME);
					TSharedRef<TPromise<TContinuationResult<void>>> Promise = MakeShared<TPromise<TContinuationResult<void>>>();
					TFuture<TContinuationResult<void>> Future = Promise->GetFuture();
					if (!XStoreProductsQueryHasMorePages(ProductQueryHandle))
					{
						XStoreCloseProductsQueryHandle(ProductQueryHandle);
						Op.SetResult({ });
						Promise->EmplaceValue(TContinuationResult<void>::Complete());
						return Future;
					}				

					TSharedRef<FGDKAsyncBlock> AsyncBlock = MakeShared<FGDKAsyncBlock>(nullptr,
						[this, Promise, WeakOperation = TWeakPtr<TOnlineAsyncOp<FCommerceQueryOffers>>(Op.AsShared())](class FGDKAsyncBlock* Block)
						{
							TSharedPtr<TOnlineAsyncOp<FCommerceQueryOffers>> Op = WeakOperation.Pin();
							if (Op && Block)
							{
								XStoreProductQueryHandle ProductQueryHandle = nullptr;
								const FCommerceQueryOffers::Params& Params = Op->GetParams();
								HRESULT Result = XStoreProductsQueryNextPageResult(*Block, &ProductQueryHandle);

								if (FAILED(Result) || !ParseProducts(Params.LocalAccountId, ProductQueryHandle, nullptr))
								{
									FOnlineError Error = Errors::FromHRESULT(Result);
									UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to query products. Error %ls", __FUNCTION__, *Error.GetLogString());
									Promise->EmplaceValue(TContinuationResult<void>::Complete());
									Op->SetError(MoveTemp(Error));
								}	
								else
								{
									Op->Data.Set<XStoreProductQueryHandle>(UE_XBL_PRODUCT_KEY_NAME, ProductQueryHandle);
								}
							}
							else
							{
								UE_LOGF(LogOnlineServices, Warning, "[%s]: Opertion removed before async return.", __FUNCTION__);
							}
							Promise->EmplaceValue(TContinuationResult<void>::Repeat());
						});
					Op.Data.Set<TSharedRef<FGDKAsyncBlock>>(UE_XBL_ASYNC_BLOCK_KEY_NAME, AsyncBlock);
					HRESULT Result = XStoreProductsQueryNextPageAsync(ProductQueryHandle, *AsyncBlock);
					if (FAILED(Result))
					{
						Promise->EmplaceValue(TContinuationResult<void>::Complete());
						FOnlineError Error = Errors::FromHRESULT(Result);
						UE_LOGF(LogOnlineServices, Warning, "[%s]: QueryProducts failed. Error %ls", __FUNCTION__, *Error.GetLogString());
						Op.SetError(MoveTemp(Error));
					}
					XStoreCloseProductsQueryHandle(ProductQueryHandle);
					return Future;				
			},FOnlineAsyncExecutionPolicy::RunOnThreadPool());
		Op->Enqueue(Services.GetParallelQueue());


	
	return Op->GetHandle();
}



TOnlineAsyncOpHandle<FCommerceQueryOffersById> FCommerceXbl::QueryOffersById(FCommerceQueryOffersById::Params&& InParams)
{
	TOnlineAsyncOpRef<FCommerceQueryOffersById> Op = GetOp<FCommerceQueryOffersById>(MoveTemp(InParams));
	if (!Op->IsReady())
	{

		const FCommerceQueryOffersById::Params& Params = Op->GetParams();
		if (!Params.LocalAccountId.IsValid())
		{
			Op->SetError(Errors::InvalidParams());
			return Op->GetHandle();
		}

		Op->Then([this](TOnlineAsyncOp<FCommerceQueryOffersById>& Op)
			{
				TSharedRef<TPromise<void>> Promise = MakeShared<TPromise<void>>();
				TFuture<void> Future = Promise->GetFuture();
				const FCommerceQueryOffersById::Params& Params = Op.GetParams();

				uint64 LocalXUID = FOnlineAccountIdRegistryXbl::Get().Find(Params.LocalAccountId);
				FGDKUserHandle GDKUserHandle = IGDKRuntimeModule::Get().GetUserHandleByXUserId(LocalXUID);

				XStoreContextHandle StoreContextHandle = GetStoreContextHandle(GDKUserHandle);

				TArray<TArray<ANSICHAR>> OfferIdsAnsiChar;
				TArray<const ANSICHAR*> OfferIdsCharPtr;

				for (int32 i = 0; i < Params.OfferIds.Num(); ++i)
				{
					TArray<ANSICHAR>& AnsiCharArray = OfferIdsAnsiChar.AddDefaulted_GetRef();
					AnsiCharArray.Append(TCHAR_TO_ANSI(*(Params.OfferIds[i])), Params.OfferIds[i].Len() + 1);
					OfferIdsCharPtr.Add(AnsiCharArray.GetData());
				}

				TSharedRef<FGDKAsyncBlock> AsyncBlock = MakeShared<FGDKAsyncBlock>(nullptr,
					[Promise, WeakOperation = TWeakPtr<TOnlineAsyncOp<FCommerceQueryOffersById>>(Op.AsShared())](class FGDKAsyncBlock* Block)
					{
						TSharedPtr<TOnlineAsyncOp<FCommerceQueryOffersById>> Op = WeakOperation.Pin();
						if (Op)
						{
							HRESULT Result = Block->GetStatus();
							if (FAILED(Result))
							{
								FOnlineError Error = Errors::FromHRESULT(Result);								
								UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to query products. Error %ls", __FUNCTION__, *Error.GetLogString());
								Op->SetError(MoveTemp(Error));
							}
						}
						else
						{
							UE_LOGF(LogOnlineServices, Warning, "[%s]: Opertion removed before async return.", __FUNCTION__);
						}
						Promise->EmplaceValue();
					});
				Op.Data.Set<TSharedRef<FGDKAsyncBlock>>(UE_XBL_ASYNC_BLOCK_KEY_NAME, AsyncBlock);

				HRESULT Result = XStoreQueryProductsAsync(StoreContextHandle, AllProductKinds, OfferIdsCharPtr.GetData(), OfferIdsCharPtr.Num(), nullptr, 0, *AsyncBlock);
				if (FAILED(Result))
				{
					Promise->EmplaceValue();
					FOnlineError Error = Errors::FromHRESULT(Result);
					UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to query products. Error %ls", __FUNCTION__, *Error.GetLogString());
					Op.SetError(MoveTemp(Error));
					return Future;
				}
				return Future;

			}, FOnlineAsyncExecutionPolicy::RunOnThreadPool()).Then([this](TOnlineAsyncOp<FCommerceQueryOffersById>& Op)
			{
				const TSharedRef<FGDKAsyncBlock>& AsyncBlock = GetOpDataChecked<TSharedRef<FGDKAsyncBlock>>(Op, UE_XBL_ASYNC_BLOCK_KEY_NAME);

				XStoreProductQueryHandle ProductQueryHandle = nullptr;
				HRESULT Result = XStoreQueryProductsResult(*AsyncBlock, &ProductQueryHandle);

				if (FAILED(Result))
				{
					FOnlineError Error = Errors::FromHRESULT(Result);
					UE_LOGF(LogOnlineServices, Warning, "[%s]: QueryProducts failed. Error %ls", __FUNCTION__, *Error.GetLogString());
					Op.SetError(MoveTemp(Error));
					return;	
				}
				
				TSharedRef<TArray<FOfferId>> OfferIds = MakeShared<TArray<FOfferId>>();

				const FCommerceQueryOffersById::Params& Params = Op.GetParams();

				if (!ParseProducts(Params.LocalAccountId,ProductQueryHandle,&*OfferIds))
				{
					FOnlineError Error = Errors::FromHRESULT(Result);
					UE_LOGF(LogOnlineServices, Warning, "[%s]: QueryProducts failed to enumerate products. Error %ls", __FUNCTION__, *Error.GetLogString());
					Op.SetError(MoveTemp(Error));
					return;
				}


				if (!XStoreProductsQueryHasMorePages(ProductQueryHandle))
				{
					XStoreCloseProductsQueryHandle(ProductQueryHandle);
					Op.SetResult({ MoveTemp(*OfferIds) });
				}
				else
				{
					Op.Data.Set<XStoreProductQueryHandle>(UE_XBL_PRODUCT_KEY_NAME, ProductQueryHandle);
					Op.Data.Set<TSharedRef<TArray<FOfferId>>>(UE_XBL_OFFERIDS_KEY_NAME, OfferIds);
				}

			}).Then([this](TOnlineAsyncOp<FCommerceQueryOffersById>& Op) mutable
			{
				XStoreProductQueryHandle ProductQueryHandle = GetOpDataChecked<XStoreProductQueryHandle>(Op, UE_XBL_PRODUCT_KEY_NAME);
				TSharedRef<TPromise<TContinuationResult<void>>> Promise = MakeShared<TPromise<TContinuationResult<void>>>();
				TFuture<TContinuationResult<void>> Future = Promise->GetFuture();
				if (!XStoreProductsQueryHasMorePages(ProductQueryHandle))
				{
					TSharedRef<TArray<FOfferId>> OfferIds = GetOpDataChecked<TSharedRef<TArray<FOfferId>>>(Op, UE_XBL_OFFERIDS_KEY_NAME);
					XStoreCloseProductsQueryHandle(ProductQueryHandle);
					Op.SetResult({ MoveTemp(*OfferIds) });
					Promise->EmplaceValue(TContinuationResult<void>::Complete());
					return Future;
				}

				TSharedRef<FGDKAsyncBlock> AsyncBlock = MakeShared<FGDKAsyncBlock>(nullptr,
					[this, Promise, WeakOperation = TWeakPtr<TOnlineAsyncOp<FCommerceQueryOffersById>>(Op.AsShared())](class FGDKAsyncBlock* Block)
					{
						TSharedPtr<TOnlineAsyncOp<FCommerceQueryOffersById>> Op = WeakOperation.Pin();
						if (Op && Block)
						{
							XStoreProductQueryHandle ProductQueryHandle = nullptr;
							const FCommerceQueryOffersById::Params& Params = Op->GetParams();
							HRESULT Result = XStoreProductsQueryNextPageResult(*Block, &ProductQueryHandle);
							TSharedRef<TArray<FOfferId>> OfferIds = GetOpDataChecked<TSharedRef<TArray<FOfferId>>>(*Op, UE_XBL_OFFERIDS_KEY_NAME);

							if (FAILED(Result) || !ParseProducts(Params.LocalAccountId, ProductQueryHandle, &*OfferIds))
							{
								FOnlineError Error = Errors::FromHRESULT(Result);
								UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to query products. Error %ls", __FUNCTION__, *Error.GetLogString());
								Op->SetError(MoveTemp(Error));
								Promise->EmplaceValue(TContinuationResult<void>::Complete());
							}
							else
							{
								Op->Data.Set<XStoreProductQueryHandle>(UE_XBL_PRODUCT_KEY_NAME, ProductQueryHandle);
							}
						}
						else
						{
							UE_LOGF(LogOnlineServices, Warning, "[%s]: Opertion removed before async return.", __FUNCTION__);
						}
						Promise->EmplaceValue(TContinuationResult<void>::Repeat());
					});
				Op.Data.Set<TSharedRef<FGDKAsyncBlock>>(UE_XBL_ASYNC_BLOCK_KEY_NAME, AsyncBlock);
				HRESULT Result = XStoreProductsQueryNextPageAsync(ProductQueryHandle, *AsyncBlock);
				if (FAILED(Result))
				{
					Promise->EmplaceValue(TContinuationResult<void>::Complete());
					FOnlineError Error = Errors::FromHRESULT(Result);
					UE_LOGF(LogOnlineServices, Warning, "[%s]: QueryProductsBy failed. Error %ls", __FUNCTION__, *Error.GetLogString());
					Op.SetError(MoveTemp(Error));
				}
				XStoreCloseProductsQueryHandle(ProductQueryHandle);
				return Future;
			}, FOnlineAsyncExecutionPolicy::RunOnThreadPool());
		Op->Enqueue(Services.GetParallelQueue());

	}
	return Op->GetHandle();
}

TOnlineResult<FCommerceGetOffers> FCommerceXbl::GetOffers(FCommerceGetOffers::Params&& Params)
{
	if (!Services.Get<FAuthXbl>()->IsLoggedIn(Params.LocalAccountId))
	{
		return TOnlineResult<FCommerceGetOffers>(Errors::NotLoggedIn());
	}
	FScopeLock ScopeLock(&CachedOffersLock);

	if (CachedOffers.Contains(Params.LocalAccountId))
	{
		TArray<FOffer> Offers;
		CachedOffers.FindChecked(Params.LocalAccountId).GenerateValueArray(Offers);
		return TOnlineResult<FCommerceGetOffers>({ MoveTemp(Offers) });
	}
	return TOnlineResult<FCommerceGetOffers>(Errors::NotFound());
}

TOnlineResult<FCommerceGetOffersById> FCommerceXbl::GetOffersById(FCommerceGetOffersById::Params&& Params)
{
	if (!Services.Get<FAuthXbl>()->IsLoggedIn(Params.LocalAccountId))
	{
		return TOnlineResult<FCommerceGetOffersById>(Errors::NotLoggedIn());
	}
	FScopeLock ScopeLock(&CachedOffersLock);

	if (CachedOffers.Contains(Params.LocalAccountId))
	{
		TArray<FOffer> Offers;
		CachedOffers.FindChecked(Params.LocalAccountId).GenerateValueArray(Offers);
		return TOnlineResult<FCommerceGetOffersById>(
			{ 
				Offers.FilterByPredicate(
				[&Params](const FOffer& Offer)
				{
					return Params.OfferIds.Contains(Offer.OfferId);
				})
			});
	}
	return TOnlineResult<FCommerceGetOffersById>(Errors::NotFound());
}

void FCommerceXbl::HandleSuspend()
{
	bStoreContextHandlesInvalidated = true;
}

TOnlineAsyncOpHandle<FCommerceShowStoreUI> FCommerceXbl::ShowStoreUI(FCommerceShowStoreUI::Params&& InParams)
{
	TOnlineAsyncOpRef<FCommerceShowStoreUI> Op = GetOp<FCommerceShowStoreUI>(MoveTemp(InParams));

	if (!Op->IsReady())
	{

		const FCommerceShowStoreUI::Params& Params = Op->GetParams();
		if (!Params.LocalAccountId.IsValid())
		{
			Op->SetError(Errors::InvalidParams());
			return Op->GetHandle();
		}

		Op->Then([this](TOnlineAsyncOp<FCommerceShowStoreUI>& Op)
			{
				const FCommerceShowStoreUI::Params& Params = Op.GetParams();

				FString StoreId;
				GConfig->GetString(IGDKRuntimeModule::Get().GetConfigSectionName(), TEXT("StoreId"), StoreId, GEngineIni);
				if (StoreId.IsEmpty())
				{
					Op.SetError(Errors::InvalidState());
					UE_LOGF(LogOnlineServices, Warning, "[%s]: No storeId found in target settings", __FUNCTION__);
					return;
				}

				uint64 LocalXUID = FOnlineAccountIdRegistryXbl::Get().Find(Params.LocalAccountId);
				FGDKUserHandle GDKUserHandle = IGDKRuntimeModule::Get().GetUserHandleByXUserId(LocalXUID);
				XStoreContextHandle StoreContextHandle = GetStoreContextHandle(GDKUserHandle);

				TUniquePtr<FGDKAsyncBlock> AsyncBlock = MakeUnique<FGDKAsyncBlock>(nullptr, [](FGDKAsyncBlock* AsyncBlock)
					{
						delete AsyncBlock;
						//we don't care about the result.
					});


				HRESULT Result = XStoreShowProductPageUIAsync(StoreContextHandle, TCHAR_TO_UTF8(*StoreId), *AsyncBlock);
				if (FAILED(Result))
				{
					AsyncBlock.Reset();
					FOnlineError Error = Errors::FromHRESULT(Result);
					UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to show store UI. Error %ls", __FUNCTION__, *Error.GetLogString());
					Op.SetError(MoveTemp(Error));
					return;
				}
				else
				{
					(void)AsyncBlock.Release();
					Op.SetResult(FCommerceShowStoreUI::Result{});
				}
			},FOnlineAsyncExecutionPolicy::RunOnThreadPool());
		Op->Enqueue(Services.GetParallelQueue());
	}
	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FCommerceQueryEntitlements> FCommerceXbl::QueryEntitlements(FCommerceQueryEntitlements::Params&& InParams)
{
	TOnlineAsyncOpRef<FCommerceQueryEntitlements> Op = GetOp<FCommerceQueryEntitlements>(MoveTemp(InParams));
	if (!Op->IsReady())
	{
		const FCommerceQueryEntitlements::Params& Params = Op->GetParams();
		if (!Params.LocalAccountId.IsValid())
		{
			Op->SetError(Errors::InvalidParams());
			return Op->GetHandle();
		}

		struct EnumerationWrapper
		{
			uint64 XUID;
			TArray<FEntitlement>* Entitlements;
		};

		Op->Then([this](TOnlineAsyncOp<FCommerceQueryEntitlements>& Op)
			{
				TSharedRef<TPromise<void>> Promise = MakeShared<TPromise<void>>();
				TFuture<void> Future = Promise->GetFuture();

				const FCommerceQueryEntitlements::Params& Params = Op.GetParams();
				if (!Services.Get<FAuthXbl>()->IsLoggedIn(Params.LocalAccountId))
				{
					Promise->EmplaceValue();
					Op.SetError(Errors::NotLoggedIn());
					return Future;
				}

				uint64 LocalXUID = FOnlineAccountIdRegistryXbl::Get().Find(Params.LocalAccountId);
				FGDKUserHandle GDKUserHandle = IGDKRuntimeModule::Get().GetUserHandleByXUserId(LocalXUID);
				XStoreContextHandle StoreContextHandle = GetStoreContextHandle(GDKUserHandle);

				TSharedRef<FGDKAsyncBlock> AsyncBlock = MakeShared<FGDKAsyncBlock>(nullptr,
					[Promise, WeakOperation = TWeakPtr<TOnlineAsyncOp<FCommerceQueryEntitlements>>(Op.AsShared())](class FGDKAsyncBlock* Block)
					{
						TSharedPtr<TOnlineAsyncOp<FCommerceQueryEntitlements>> Op = WeakOperation.Pin();
						if (Op)
						{
							HRESULT Result = Block->GetStatus();
							if (FAILED(Result))
							{
								FOnlineError Error = Errors::FromHRESULT(Result);
								UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to query entitlements. Error %ls", __FUNCTION__, *Error.GetLogString());
								Op->SetError(MoveTemp(Error));
							}
						}
						else
						{
							UE_LOGF(LogOnlineServices, Warning, "[%s]: Opertion removed before async return.", __FUNCTION__);
						}
						Promise->EmplaceValue();
					});
				Op.Data.Set<TSharedRef<FGDKAsyncBlock>>(UE_XBL_ASYNC_BLOCK_KEY_NAME, AsyncBlock);

				HRESULT Result = XStoreQueryEntitledProductsAsync(StoreContextHandle, AllProductKinds, UINT32_MAX, *AsyncBlock);
				if(FAILED(Result))
				{
					FOnlineError Error = Errors::FromHRESULT(Result);
					UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to query entitlements. Error %ls", __FUNCTION__, *Error.GetLogString());
					Op.SetError(MoveTemp(Error));
					Promise->EmplaceValue();
					return Future;
				}
				return Future;
			}, FOnlineAsyncExecutionPolicy::RunOnThreadPool()).Then([this](TOnlineAsyncOp<FCommerceQueryEntitlements>& Op)
			{
				const TSharedRef<FGDKAsyncBlock>& AsyncBlock = GetOpDataChecked<TSharedRef<FGDKAsyncBlock>>(Op, UE_XBL_ASYNC_BLOCK_KEY_NAME);

				XStoreProductQueryHandle QueryHandle = nullptr;
				HRESULT Result = XStoreQueryEntitledProductsResult(*AsyncBlock, &QueryHandle);

				if (FAILED(Result))
				{
					FOnlineError Error = Errors::FromHRESULT(Result);
					UE_LOGF(LogOnlineServices, Warning, "[%s]: Query entitlements failed. Error %ls", __FUNCTION__, *Error.GetLogString());
					Op.SetError(MoveTemp(Error));
					return;
				}
				const FCommerceQueryEntitlements::Params& Params = Op.GetParams();
			
				
				TArray<FEntitlement> FoundEntitlements;
				EnumerationWrapper Wrapper{ FOnlineAccountIdRegistryXbl::Get().Find(Params.LocalAccountId),&FoundEntitlements };
			
				Result = XStoreEnumerateProductsQuery(QueryHandle, &Wrapper, [](const XStoreProduct* Product, void* Context)
					{
						EnumerationWrapper* Wrapper = static_cast<EnumerationWrapper*>(Context);
						if (Product)
						{
							Wrapper->Entitlements->Emplace(XStoreProductToFEntitlement(Product, Wrapper->XUID));
						}
						return true;
					});

				if (FAILED(Result))
				{
					FOnlineError Error = Errors::FromHRESULT(Result);
					UE_LOGF(LogOnlineServices, Warning, "[%s]: Query entitlements failed to enumerate products. Error %ls", __FUNCTION__, *Error.GetLogString());
					Op.SetError(MoveTemp(Error));
					return;
				}	

				FScopeLock ScopeLock(&CachedEntitlementsLock);
				for (FEntitlement& Found : FoundEntitlements)
				{
					CachedEntitlements.FindOrAdd(Params.LocalAccountId).Emplace(Found.EntitlementId, MoveTemp(Found));
				}

				if (!XStoreProductsQueryHasMorePages(QueryHandle))
				{
					XStoreCloseProductsQueryHandle(QueryHandle);
					Op.SetResult(FCommerceQueryEntitlements::Result{});
				}
				else
				{
					Op.Data.Set<XStoreProductQueryHandle>(UE_XBL_PRODUCT_KEY_NAME, QueryHandle);
				}

				}).Then([this](TOnlineAsyncOp<FCommerceQueryEntitlements>& Op) mutable
					{
						XStoreProductQueryHandle ProductQueryHandle = GetOpDataChecked<XStoreProductQueryHandle>(Op, UE_XBL_PRODUCT_KEY_NAME);
						TSharedRef<TPromise<TContinuationResult<void>>> Promise = MakeShared<TPromise<TContinuationResult<void>>>();
						TFuture<TContinuationResult<void>> Future = Promise->GetFuture();
						if (!XStoreProductsQueryHasMorePages(ProductQueryHandle))
						{
							Promise->EmplaceValue(TContinuationResult<void>::Complete());
							XStoreCloseProductsQueryHandle(ProductQueryHandle);
							Op.SetResult({ });
							return Future;
						}

						TSharedRef<FGDKAsyncBlock> AsyncBlock = MakeShared<FGDKAsyncBlock>(nullptr,
							[this, Promise, WeakOperation = TWeakPtr<TOnlineAsyncOp<FCommerceQueryEntitlements>>(Op.AsShared())](class FGDKAsyncBlock* Block)
							{
								TSharedPtr<TOnlineAsyncOp<FCommerceQueryEntitlements>> Op = WeakOperation.Pin();
								if (Op && Block)
								{
									XStoreProductQueryHandle ProductQueryHandle = nullptr;
									const FCommerceQueryEntitlements::Params& Params = Op->GetParams();
									HRESULT Result = XStoreProductsQueryNextPageResult(*Block, &ProductQueryHandle);
									TArray<FEntitlement> FoundEntitlements;
									EnumerationWrapper Wrapper{ FOnlineAccountIdRegistryXbl::Get().Find(Params.LocalAccountId),&FoundEntitlements };

									if (FAILED(Result) || FAILED(XStoreEnumerateProductsQuery(ProductQueryHandle, &Wrapper, [](const XStoreProduct* Product, void* Context)
										{
											EnumerationWrapper* Wrapper = static_cast<EnumerationWrapper*>(Context);
											if (Product)
											{
												Wrapper->Entitlements->Emplace(XStoreProductToFEntitlement(Product, Wrapper->XUID));
											}
											return true;
										})))
									{
										FOnlineError Error = Errors::FromHRESULT(Result);
										UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to query products. Error %ls", __FUNCTION__, *Error.GetLogString());
										Op->SetError(MoveTemp(Error));
										Promise->EmplaceValue(TContinuationResult<void>::Complete());
									}
									else
									{
										FScopeLock ScopeLock(&CachedEntitlementsLock);

										for (FEntitlement& Found : FoundEntitlements)
										{
											CachedEntitlements.FindOrAdd(Params.LocalAccountId).Emplace(Found.EntitlementId, MoveTemp(Found));
										}
										Op->Data.Set<XStoreProductQueryHandle>(UE_XBL_PRODUCT_KEY_NAME, ProductQueryHandle);
									}
								}
								else
								{
									UE_LOGF(LogOnlineServices, Warning, "[%s]: Opertion removed before async return.", __FUNCTION__);
								}
								Promise->EmplaceValue(TContinuationResult<void>::Repeat());
							});
						Op.Data.Set<TSharedRef<FGDKAsyncBlock>>(UE_XBL_ASYNC_BLOCK_KEY_NAME, AsyncBlock);
						HRESULT Result = XStoreProductsQueryNextPageAsync(ProductQueryHandle, *AsyncBlock);
						if (FAILED(Result))
						{
							Promise->EmplaceValue(TContinuationResult<void>::Complete());
							FOnlineError Error = Errors::FromHRESULT(Result);
							UE_LOGF(LogOnlineServices, Warning, "[%s]: QueryProducts failed. Error %ls", __FUNCTION__, *Error.GetLogString());
							Op.SetError(MoveTemp(Error));
						}
						XStoreCloseProductsQueryHandle(ProductQueryHandle);
						return Future;
					}, FOnlineAsyncExecutionPolicy::RunOnThreadPool());
			Op->Enqueue(Services.GetParallelQueue());
	}
	return Op->GetHandle();
}


TOnlineResult<FCommerceGetEntitlements> FCommerceXbl::GetEntitlements(FCommerceGetEntitlements::Params&& Params)
{
	if (!Services.Get<FAuthXbl>()->IsLoggedIn(Params.LocalAccountId))
	{
		return TOnlineResult<FCommerceGetEntitlements>(Errors::NotLoggedIn());
	}
	FScopeLock ScopeLock(&CachedEntitlementsLock);

	if (CachedEntitlements.Contains(Params.LocalAccountId))
	{
		TArray<FEntitlement> Entitlements;
		CachedEntitlements.FindChecked(Params.LocalAccountId).GenerateValueArray(Entitlements);
		return TOnlineResult<FCommerceGetEntitlements>({ MoveTemp(Entitlements) });
	}
	return TOnlineResult<FCommerceGetEntitlements>(Errors::NotFound());
}


TOnlineAsyncOpHandle<FCommerceCheckout> FCommerceXbl::Checkout(FCommerceCheckout::Params&& InParams)
{	// There is no way to make a purchase directly, best we can do is show the platform storeUI for the selected purchase.
	TOnlineAsyncOpRef<FCommerceCheckout> Op = GetOp<FCommerceCheckout>(MoveTemp(InParams));

	if (!Op->IsReady())
	{
		const FCommerceCheckout::Params& Params = Op->GetParams();
		if (!Params.LocalAccountId.IsValid())
		{
			Op->SetError(Errors::InvalidParams());
			return Op->GetHandle();
		}

		Op->Then([this](TOnlineAsyncOp<FCommerceCheckout>& Op)
			{
				TSharedRef<TPromise<void>> Promise = MakeShared<TPromise<void>>();
				TFuture<void> Future = Promise->GetFuture();
				const FCommerceCheckout::Params& Params = Op.GetParams();
				if (!Services.Get<FAuthXbl>()->IsLoggedIn(Params.LocalAccountId))
				{
					Op.SetError(Errors::NotLoggedIn());
					Promise->EmplaceValue();
					return Future;
				}

				if (Params.Offers.Num()==0)
				{
					Op.SetError(Errors::InvalidParams());
					Promise->EmplaceValue();
					return Future;
				}
				
				UE_CLOGF(Params.Offers.Num() > 1,LogOnlineServices, Warning, "[%s]: Attempting to show purchase UI for more than 1 item. Xbox does not support this", __FUNCTION__);
				
				uint64 LocalXUID = FOnlineAccountIdRegistryXbl::Get().Find(Params.LocalAccountId);
				FGDKUserHandle GDKUserHandle = IGDKRuntimeModule::Get().GetUserHandleByXUserId(LocalXUID);
				XStoreContextHandle StoreContextHandle = GetStoreContextHandle(GDKUserHandle);

				TSharedRef<FGDKAsyncBlock> AsyncBlock = MakeShared<FGDKAsyncBlock>(nullptr, [Promise](class FGDKAsyncBlock* Block)
					{
						HRESULT Result = Block->GetStatus();
						if (FAILED(Result))
						{
							FOnlineError Error = Errors::FromHRESULT(Result);
							UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to show store UI. Error %ls", __FUNCTION__, *Error.GetLogString());
						}
						Promise->EmplaceValue();
					});
				Op.Data.Set<TSharedRef<FGDKAsyncBlock>>(UE_XBL_ASYNC_BLOCK_KEY_NAME, AsyncBlock);


					// We can only show the UI for one item, so show for the first offer. 				
					HRESULT Result = XStoreShowPurchaseUIAsync(StoreContextHandle, TCHAR_TO_UTF8(*Params.Offers[0].OfferId), nullptr, nullptr, *AsyncBlock);
					if (FAILED(Result))
					{
						FOnlineError Error = Errors::FromHRESULT(Result);
						UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to show store UI. Error %ls", __FUNCTION__, *Error.GetLogString());
						Promise->EmplaceValue();
						Op.SetError(MoveTemp(Error));
						return Future;
					}
					return Future;				
				},
				FOnlineAsyncExecutionPolicy::RunOnThreadPool()).Then([this](TOnlineAsyncOp<FCommerceCheckout>& Op)
					{
						const TSharedRef<FGDKAsyncBlock>& AsyncBlock = GetOpDataChecked<TSharedRef<FGDKAsyncBlock>>(Op, UE_XBL_ASYNC_BLOCK_KEY_NAME);

						HRESULT Result = XStoreShowPurchaseUIResult(*AsyncBlock);
						if (FAILED(Result))
						{
							FOnlineError Error = Errors::FromHRESULT(Result);
							UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to show store UI. Error %ls", __FUNCTION__, *Error.GetLogString());
							Op.SetError(MoveTemp(Error));
							return;
						}
						Op.SetResult(FCommerceCheckout::Result{});
					});
		Op->Enqueue(Services.GetParallelQueue());
	}
	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FCommerceRedeemEntitlement> FCommerceXbl::RedeemEntitlement(FCommerceRedeemEntitlement::Params&& InParams)
{
	TOnlineAsyncOpRef<FCommerceRedeemEntitlement> Op = GetOp<FCommerceRedeemEntitlement>(MoveTemp(InParams));

	if (!Op->IsReady())
	{
		const FCommerceRedeemEntitlement::Params& Params = Op->GetParams();
		if (!Params.LocalAccountId.IsValid())
		{
			Op->SetError(Errors::InvalidParams());
			return Op->GetHandle();
		}

		Op->Then([this](TOnlineAsyncOp<FCommerceRedeemEntitlement>& Op)
			{

				TSharedRef<TPromise<void>> Promise = MakeShared<TPromise<void>>();
				TFuture<void> Future = Promise->GetFuture();
				const FCommerceRedeemEntitlement::Params& Params = Op.GetParams();
				if (!Services.Get<FAuthXbl>()->IsLoggedIn(Params.LocalAccountId))
				{
					Promise->EmplaceValue();
					Op.SetError(Errors::NotLoggedIn());
					return Future;
				}
				FScopeLock ScopeLock(&CachedEntitlementsLock);

				if (!CachedEntitlements.Contains(Params.LocalAccountId))
				{
					Promise->EmplaceValue();
					Op.SetError(Errors::InvalidUser());
					return Future;					
				}
				if (!CachedEntitlements[Params.LocalAccountId].Contains(Params.EntitlementId))
				{
					Promise->EmplaceValue();
					Op.SetError(Errors::InvalidParams());
					return Future;
				}

				TSharedRef<FGDKAsyncBlock> AsyncBlock = MakeShared<FGDKAsyncBlock>(nullptr,
					[Promise, WeakOperation = TWeakPtr<TOnlineAsyncOp<FCommerceRedeemEntitlement>>(Op.AsShared())](class FGDKAsyncBlock* Block)
					{
						TSharedPtr<TOnlineAsyncOp<FCommerceRedeemEntitlement>> Op = WeakOperation.Pin();
						if (Op)
						{
							HRESULT Result = Block->GetStatus();
							if (FAILED(Result))
							{
								FOnlineError Error = Errors::FromHRESULT(Result);
								UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed redeem entitlement. Error %ls", __FUNCTION__, *Error.GetLogString());
								Op->SetError(MoveTemp(Error));
							}
						}
						else
						{
							UE_LOGF(LogOnlineServices, Warning, "[%s]: Opertion removed before async return.", __FUNCTION__);
						}
						Promise->EmplaceValue();
						delete Block;
					});
				
				FString ProductId = CachedEntitlements[Params.LocalAccountId][Params.EntitlementId].ProductId;
				uint64 LocalXUID = FOnlineAccountIdRegistryXbl::Get().Find(Params.LocalAccountId);
				FGDKUserHandle GDKUserHandle = IGDKRuntimeModule::Get().GetUserHandleByXUserId(LocalXUID);
				XStoreContextHandle StoreContextHandle = GetStoreContextHandle(GDKUserHandle);
				GUID TrackingID = {}; //CDA used for tracking redeem events on MS service, should we be retaining this ?
				HRESULT Result = XStoreReportConsumableFulfillmentAsync(
					StoreContextHandle, TCHAR_TO_UTF8(*ProductId), 
					Params.Quantity,
					TrackingID,
					*AsyncBlock);

				if (FAILED(Result))
				{
					Promise->EmplaceValue();
					FOnlineError Error = Errors::FromHRESULT(Result);
					UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed redeem entitlement. Error %ls", __FUNCTION__, *Error.GetLogString());
					Op.SetError(MoveTemp(Error));
					return Future;
				}
				return Future;
			},FOnlineAsyncExecutionPolicy::RunOnThreadPool());
		Op->Enqueue(Services.GetParallelQueue());
	}
	return Op->GetHandle();
}

void FCommerceXbl::CleanupContextHandles()
{
	FScopeLock ScopeLock(&StoreContextsLock);

	UE_LOGF(LogOnlineServices, Verbose, "[%s]: Closing all context handles", __FUNCTION__);

	for (TPair<FGDKUserHandle, XStoreContextHandle>& Entry : StoreContexts)
	{
		if (Entry.Value != nullptr)
		{
			// XStoreCloseContextHandle IS safe to call on a time-sensitive thread
			XStoreCloseContextHandle(Entry.Value);
			Entry.Value = nullptr;
		}
	}

	StoreContexts.Reset();
}

XStoreContextHandle FCommerceXbl::GetStoreContextHandle(FGDKUserHandle GDKUser)
{
	if (bStoreContextHandlesInvalidated)
	{
		CleanupContextHandles();
		bStoreContextHandlesInvalidated = false;
	}

	FScopeLock ScopeLock(&StoreContextsLock);

	// XStoreCreateContext is not safe to call on a time-sensitive thread, call it within async task only.
	check(!IsInGameThread());

	XStoreContextHandle Result = nullptr;

	if (StoreContexts.Contains(GDKUser))
	{
		Result = StoreContexts[GDKUser];
	}
	else
	{
		if (XStoreCreateContext(GDKUser, &Result) == S_OK && Result != nullptr)
		{
			StoreContexts.Add(GDKUser, Result);
		}
	}

	return Result;
}


} // namespace UE::Online
#endif // WITH_GRDK
