// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK

#include "Online/ConnectivityXbl.h"
#include "Online/OnlineUtils.h"
#include "Online/Windows/WindowsOnlineErrorDefinitions.h"
#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#include "Misc/CoreDelegates.h"

THIRD_PARTY_INCLUDES_START
#include <XGameRuntimeFeature.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

namespace UE::Online {

void FConnectivityXbl::PostInitialize()
{
	//GDK runtime is broadcasting Xbox network changes on the core delegate.
	FCoreDelegates::OnNetworkConnectionStatusChanged.AddRaw(this, &FConnectivityXbl::OnNetworkConnectionStatusChanged);
	ConnectionStatus = FGenericPlatformMisc::GetNetworkConnectionStatus() == ENetworkConnectionStatus::Connected ? EOnlineServicesConnectionStatus::Connected : EOnlineServicesConnectionStatus::NotConnected;
}
void FConnectivityXbl::PreShutdown()
{
	FCoreDelegates::OnNetworkConnectionStatusChanged.RemoveAll(this);
}


void FConnectivityXbl::OnNetworkConnectionStatusChanged(ENetworkConnectionStatus LastConnectionState, ENetworkConnectionStatus NewConnectionState)
{
	EOnlineServicesConnectionStatus OldConnectionStatus = ConnectionStatus;
	ConnectionStatus = NewConnectionState == ENetworkConnectionStatus::Connected ? EOnlineServicesConnectionStatus::Connected : EOnlineServicesConnectionStatus::NotConnected;
	UE_LOGF(LogOnlineServices, Log, "[%s]: %ls", __FUNCTION__, LexToString(ConnectionStatus));

	if(ConnectionStatus == OldConnectionStatus)
	{
		return;
	}

	FConnectionStatusChanged Params;
	Params.ServiceName = Services.GetServiceConfigName();
	Params.PreviousStatus = OldConnectionStatus;
	Params.CurrentStatus = ConnectionStatus;
	OnConnectionStatusChangedEvent.Broadcast(Params);
}



/* UE::Online */}

#endif // WITH_GRDK

