// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "Online/EventsXbl.h"
#include "Serialization/JsonSerializerMacros.h"
#include "GDKThreadCheck.h"
#include "Online/OnlineServicesXbl.h"
#include "Online/OnlineServicesLog.h"
#include "Online/AuthXbl.h"
#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <xsapi-c/events_c.h>
#include <XGameRuntimeFeature.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

namespace UE::Online {

bool FEventsXbl::TriggerEvent(const FAccountId& PlayerId, const TCHAR* EventName, const FOnlineEventParams& EventParams)
{
	if (XGameRuntimeIsFeatureAvailable(XGameRuntimeFeature::XGameEvent) == false)
	{
		UE_LOGF(LogOnlineServices, Warning, "[%s]: Event System not Available.", __FUNCTION__);
		return false;
	}

	if (!PlayerId.IsValid())
	{
		UE_LOGF(LogOnlineServices, Warning, "[%s]: Invalid AccountID.", __FUNCTION__);
		return false;
	}

	if (!Service->Get<FAuthXbl>()->IsLoggedIn(PlayerId))
	{
		UE_LOGF(LogOnlineServices, Warning, "[%s]: Account not signed in.", __FUNCTION__);
		return false;
	}

	uint64 XUID = FOnlineAccountIdRegistryXbl::Get().Find(PlayerId);
	if (XUID == 0)
	{
		UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to find XUID for user %ls.", __FUNCTION__, *ToString(PlayerId));
		return false;
	}

	if (EventName == nullptr)
	{
		UE_LOGF(LogOnlineServices, Warning, "[%s]: Invalid Event Name.", __FUNCTION__);
		return false;
	}

	FGDKContextHandle GDKContext = Service->ContextManager->GetGDKContext(FOnlineAccountIdRegistryXbl::Get().Find(PlayerId));

	if (!GDKContext.IsValid())
	{
		UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to find find GDK context for user %ls.", __FUNCTION__, *ToString(PlayerId));
		return false;
	}

	TSharedRef<FJsonObject> JsonObject(new FJsonObject());

	for (const TPair< FString, FSchemaVariant >& Pair : EventParams)
	{
		ESchemaAttributeType ValueType = Pair.Value.GetType();
		const FString& FieldName = Pair.Key;
		switch (ValueType)
		{
		case ESchemaAttributeType::Int64:
		{			
			JsonObject->SetNumberField(FieldName, static_cast<double>(Pair.Value.GetInt64()));
			break;
		}
		case ESchemaAttributeType::Double:
		{
			JsonObject->SetNumberField(FieldName, Pair.Value.GetDouble());
			break;
		}
		case ESchemaAttributeType::String:
		{
			JsonObject->SetStringField(FieldName, Pair.Value.GetString());
			break;
		}
		case ESchemaAttributeType::Bool:
		{
			JsonObject->SetBoolField(FieldName, Pair.Value.GetBoolean());
			break;
		}
		default:
		{
			UE_LOGF(LogOnlineServices, Warning, "[%s]: Unsupported argument type in event [%ls] for field [%ls].", __FUNCTION__, EventName, *FieldName);
			return false;
		}
		}
	}

	FString DimensionsJson;
	auto JsonWriter = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR> >::Create(&DimensionsJson);
	FJsonSerializer::Serialize(JsonObject, JsonWriter);
	JsonWriter->Close();

	GDK_SCOPE_NOT_TIME_SENSITIVE(); // XblEventsWriteInGameEvent calls XGameEventWrite which is not safe to call on time-sensitive threads
	HRESULT Result = XblEventsWriteInGameEvent(GDKContext, TCHAR_TO_UTF8(EventName), TCHAR_TO_UTF8(*DimensionsJson), "{}");
	if (Result != S_OK)
	{
		UE_LOGF(LogOnlineServices, VeryVerbose, "[%s]: XblEventsWriteInGameEvent Failed for event: %ls.", __FUNCTION__, EventName);
		return false;
	}
	UE_LOGF(LogOnlineServices, VeryVerbose, "[%s]: XblEventsWriteInGameEvent Succeeded for event: %ls.", __FUNCTION__, EventName);
	return true;
}

} // namespace UE::Online

#endif //WITH_GRDK
