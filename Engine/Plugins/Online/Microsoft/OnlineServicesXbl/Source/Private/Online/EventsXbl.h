// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#if WITH_GRDK

#include "Online/CoreOnline.h"
#include "Online/SchemaTypes.h"
#include "GDKHandle.h"
#include "GDKRuntimeModule.h"
#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#include "Online/OnlineComponent.h"
THIRD_PARTY_INCLUDES_START
#include <xsapi-c/types_c.h>
#include <xsapi-c/xbox_live_global_c.h>
#include <xsapi-c/xbox_live_context_c.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"
//#include "SimpleTokenParser.h"

namespace UE::Online {
	class FOnlineServicesXbl;
	typedef  TMap<FString, FSchemaVariant> FOnlineEventParams;


/**
 *	FEventsXbl - Internal class for events
 */
//class FEventsXbl : public IOnlineEvents
class  FEventsXbl : public TSharedFromThis<FEventsXbl, ESPMode::ThreadSafe>
{
public:
	FEventsXbl(FOnlineServicesXbl* InService) { Service = InService; };

	virtual ~FEventsXbl() = default;


	/**
	 * Calls an event
	 *
	 * @param PlayerId	- Id of the player to call the event on
	 * @param Name		- Name of requested event
	 * @param Parms		- Parameters that will be passed directly to the event
	 */
	virtual bool TriggerEvent(const FAccountId& PlayerId, const TCHAR* EventName, const FOnlineEventParams& EventParams);
private:	
	FOnlineServicesXbl* Service;
};

typedef TSharedPtr<FEventsXbl, ESPMode::ThreadSafe> FEventsXblPtr;
} // namespace UE::Online
#endif // WITH_GRDK
