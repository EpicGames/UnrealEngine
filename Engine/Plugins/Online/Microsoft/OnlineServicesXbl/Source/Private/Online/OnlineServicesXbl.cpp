// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK

#include "Online/OnlineServicesXbl.h"

#include "Online/AuthXbl.h"
#include "Online/ConnectivityXbl.h"
#include "Online/UserInfoXbl.h"
#include "Online/PresenceXbl.h"
#include "Online/SocialXbl.h"
#include "Online/PrivilegesXbl.h"
#include "Online/CommerceXbl.h"
#include "Online/TitleFileXbl.h"
#include "Online/UserFileXbl.h"
#include "Online/AchievementsXbl.h"
#include "Online/StatsXbl.h"
#include "Online/EventsXbl.h"
#include "GDKRuntimeModule.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <httpClient/httpProvider.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"


namespace UE::Online {

struct FOnlineServicesXblConfig
{
	TArray<FString> RetailSandboxes;
	TArray<FString> CertSandboxes;

};

namespace Meta
{
	BEGIN_ONLINE_STRUCT_META(FOnlineServicesXblConfig)
		ONLINE_STRUCT_FIELD(FOnlineServicesXblConfig, RetailSandboxes),
		ONLINE_STRUCT_FIELD(FOnlineServicesXblConfig, CertSandboxes)
		END_ONLINE_STRUCT_META()
}		/* Meta */


FOnlineServicesXbl::FOnlineServicesXbl(FName InInstanceName, FName InInstanceConfigName)
	: FOnlineServicesCommon(TEXT("Xbox"), InInstanceName, InInstanceConfigName)
	, ContextManager(new FUserContextsManagerXbl(this))
	, EventLauncher( new FEventsXbl(this))
{
}

void FOnlineServicesXbl::RegisterComponents()
{
	Components.Register<FAuthXbl>(*this);
	Components.Register<FConnectivityXbl>(*this);
	Components.Register<FUserInfoXbl>(*this);
	Components.Register<FPresenceXbl>(*this);
	Components.Register<FSocialXbl>(*this);
	Components.Register<FPrivilegesXbl>(*this);
	Components.Register<FCommerceXbl>(*this);
	Components.Register<FTitleFileXbl>(*this);
	Components.Register<FUserFileXbl>(*this);
	Components.Register<FAchievementsXbl>(*this);
	Components.Register<FStatsXbl>(*this);

	FOnlineServicesCommon::RegisterComponents();
}

void FOnlineServicesXbl::Initialize()
{
	// need to initialize xbl before service init because auth init triggers context creation (an xbl function)
	InitializeXblByConfig(); 
	Super::Initialize();
}
void FOnlineServicesXbl::PostInitialize()
{
	Super::PostInitialize();

	ServiceCallRoutedHandlerContext = XblAddServiceCallRoutedHandler(
		[](XblServiceCallRoutedArgs args, void* context)
		{
			const char* HttpMethod = nullptr;
			const char* HttpUrl = nullptr;
			HCHttpCallRequestGetUrl(args.call, &HttpMethod, &HttpUrl);

			const char* HttpRequestBody = nullptr;
			HCHttpCallRequestGetRequestBodyString(args.call, &HttpRequestBody);

			UE_LOGF(LogOnlineServices, Verbose, "[URL]: %ls %ls", UTF8_TO_TCHAR(HttpMethod), UTF8_TO_TCHAR(HttpUrl));
			if (HttpRequestBody)
			{
				UE_LOGF(LogOnlineServices, Verbose, "[RequestBody]: %ls", UTF8_TO_TCHAR(HttpRequestBody));
			}
			UE_LOGF(LogOnlineServices, Verbose, "");
			UE_LOGF(LogOnlineServices, Verbose, "[Response]: %ls", UTF8_TO_TCHAR(args.fullResponseFormatted));
			UE_LOGF(LogOnlineServices, Verbose, "");
		}
	, nullptr);
}
void FOnlineServicesXbl::PreShutdown()
{
	XblRemoveServiceCallRoutedHandler(ServiceCallRoutedHandlerContext);
	ServiceCallRoutedHandlerContext = 0;
	Super::PreShutdown();
}

EOnlineEnvironment FOnlineServicesXbl::GetOnlineEnvironment() const
{
	FOnlineServicesXblConfig Config;
	LoadConfig(Config);

	FString SandboxIdString(IGDKRuntimeModule::Get().GetXboxSandboxId());
	if (SandboxIdString.Equals(TEXT("RETAIL")) || Config.RetailSandboxes.Contains(SandboxIdString))
	{
		return EOnlineEnvironment::Production;
	}

	if (SandboxIdString.StartsWith(TEXT("CERT")) || SandboxIdString.EndsWith(TEXT(".99")) || SandboxIdString.EndsWith(TEXT(".98")) || Config.CertSandboxes.Contains(SandboxIdString))
	{
		return EOnlineEnvironment::Certification;
	}

	return EOnlineEnvironment::Development;
}

void FOnlineServicesXbl::InitializeXblByConfig()
{
	UTF8CHAR ServiceConfigurationID[XBL_SCID_LENGTH];
	ZeroMemory(ServiceConfigurationID, XBL_SCID_LENGTH);
	FPlatformString::Convert(ServiceConfigurationID, XBL_SCID_LENGTH, *IGDKRuntimeModule::Get().GetPrimaryServiceConfigId());

	XblInitArgs xblInit = {nullptr, (const char*)ServiceConfigurationID };
	HRESULT Result = XblInitialize(&xblInit);
	check(SUCCEEDED(Result));
}

/* UE::Online */ }

#endif // WITH_GRDK
