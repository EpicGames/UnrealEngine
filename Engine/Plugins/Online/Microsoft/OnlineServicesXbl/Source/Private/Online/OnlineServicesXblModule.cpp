// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK

#include "Modules/ModuleManager.h"

#include "Online/OnlineServicesRegistry.h"
#include "Online/OnlineServicesXbl.h"
#include "Online/AuthXbl.h"

namespace UE::Online
{

class FOnlineServicesXblModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
protected:
};

class FOnlineServicesFactoryXbl : public IOnlineServicesFactory
{
public:
	virtual ~FOnlineServicesFactoryXbl() {}
	virtual TSharedPtr<IOnlineServices> Create(FName InInstanceName, FName InInstanceConfigName) override
	{
		return MakeShared<FOnlineServicesXbl>(InInstanceName, InInstanceConfigName);
	}
protected:
};

void FOnlineServicesXblModule::StartupModule()
{
	// Making sure we load the module at this point will avoid errors while cooking
	const FName OnlineServicesInterfaceModuleName = TEXT("OnlineServicesInterface");
	if (!FModuleManager::Get().IsModuleLoaded(OnlineServicesInterfaceModuleName))
	{
		FModuleManager::Get().LoadModuleChecked(OnlineServicesInterfaceModuleName);
	}

	FOnlineServicesRegistry::Get().RegisterServicesFactory(EOnlineServices::Xbox, MakeUnique<FOnlineServicesFactoryXbl>());
	FOnlineIdRegistryRegistry::Get().RegisterAccountIdRegistry(EOnlineServices::Xbox, &FOnlineAccountIdRegistryXbl::Get());
}

void FOnlineServicesXblModule::ShutdownModule()
{
	FOnlineServicesRegistry::Get().UnregisterServicesFactory(EOnlineServices::Xbox);
	FOnlineIdRegistryRegistry::Get().UnregisterAccountIdRegistry(EOnlineServices::Xbox);
}

/* UE::Online */ }

IMPLEMENT_MODULE(UE::Online::FOnlineServicesXblModule, OnlineServicesXbl);

#endif // WITH_GRDK
