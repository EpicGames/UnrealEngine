// Copyright Epic Games, Inc. All Rights Reserved.
#if WITH_GRDK

#include "Online/PresenceXbl.h"
#include "Online/AuthXbl.h"
#include "Online/OnlineBase.h"
#include "Online/OnlineServicesXbl.h"
#include "Online/OnlineUtilsCommon.h"
#include "Online/OnlineResult.h"
#include "Templates/ValueOrError.h"
#include "GDKRuntimeModule.h"
#include "GDKHandle.h"
#include "Online/Windows/WindowsOnlineErrorDefinitions.h"
THIRD_PARTY_INCLUDES_START
#include <xsapi-c/presence_c.h>
THIRD_PARTY_INCLUDES_END

namespace UE::Online {

	EUserPresenceStatus ToPresenceStatus(const XblPresenceUserState& OnlineStatus)
	{
		switch (OnlineStatus)
		{
		case XblPresenceUserState::Online:
			return EUserPresenceStatus::Online;
		case XblPresenceUserState::Away:
			return EUserPresenceStatus::Away;
		case XblPresenceUserState::Offline:
			return EUserPresenceStatus::Offline;
		case XblPresenceUserState::Unknown:
		default:
			return EUserPresenceStatus::Unknown;
		};
	}

	void ParsePresenceFromDeviceRecords(TSharedRef<FUserPresence>& Presence, const XblPresenceDeviceRecord* UserDeviceRecords, const uint64 NumDeviceRecords, const uint32 TitleId)
	{
		Presence->GameStatus = EUserPresenceGameStatus::Unknown;
		Presence->RichPresenceString = FString();
		// We get recored for all devices the user is associated with. Each of those devices can have multiple active titles.
		// We have to enumerate through all of them to look for our own.
		for (uint64 DeviceIndex = 0; DeviceIndex < NumDeviceRecords; ++DeviceIndex)
		{
			uint64 TitleRecords = UserDeviceRecords[DeviceIndex].titleRecordsCount;
			for (uint64 TitleIndex = 0; TitleIndex < TitleRecords; ++TitleIndex)
			{
				const XblPresenceTitleRecord& TitleRecord = UserDeviceRecords[DeviceIndex].titleRecords[TitleIndex];
				if (TitleId == TitleRecord.titleId)
				{
					Presence->GameStatus = EUserPresenceGameStatus::PlayingThisGame;
					Presence->RichPresenceString = UTF8_TO_TCHAR(TitleRecord.richPresenceString);
				}
				else
				{
					Presence->GameStatus = EUserPresenceGameStatus::PlayingOtherGame;
				}
			}
		}
	}
	FPresenceXbl::FPresenceXbl(FOnlineServicesXbl& InServices)
		: FPresenceCommon(InServices)
		, Services(InServices)
	{

	}

	void FPresenceXbl::Initialize()
	{
		TitleId = IGDKRuntimeModule::Get().GetTitleId();
		FPresenceCommon::Initialize();

		TitleStatusUpdatedHandle = static_cast<FOnlineServicesXbl&>(Services).ContextManager->OnTitleStatusUpdate().Add(this, &FPresenceXbl::TitleStatusUpdate);
		OnlineStatusUpdateHandle = static_cast<FOnlineServicesXbl&>(Services).ContextManager->OnOnlineStatusUpdate().Add(this, &FPresenceXbl::OnlineStatusUpdate);

	}

	void FPresenceXbl::PreShutdown()
	{
		FPresenceCommon::PreShutdown();

		TitleStatusUpdatedHandle.Unbind();
		OnlineStatusUpdateHandle.Unbind();

	}

	void FPresenceXbl::Tick(float DeltaSeconds)
	{
	}



	/** Get a user's presence, creating entries if missing */
	TSharedRef<FUserPresence> FPresenceXbl::FindOrCreatePresence(FAccountId LocalAccountId, FAccountId PresenceAccountId)
	{
		TMap<FAccountId, TSharedRef<FUserPresence>>& LocalUserPresenceList = PresenceLists.FindOrAdd(LocalAccountId);
		if (const TSharedRef<FUserPresence>* const ExistingPresence = LocalUserPresenceList.Find(PresenceAccountId))
		{
			return *ExistingPresence;
		}

		TSharedRef<FUserPresence> UserPresence = MakeShared<FUserPresence>();
		UserPresence->AccountId = PresenceAccountId;
		LocalUserPresenceList.Emplace(PresenceAccountId, UserPresence);
		return UserPresence;
	}

	void FPresenceXbl::FindPresenceEntriesAndObservingLocalUsers(FAccountId PresenceAccountId, TArray<TSharedRef<FUserPresence>>& OutEntries, TArray<FAccountId>& OutObservers)
	{
		for (TPair<FAccountId, TMap<FAccountId, TSharedRef<FUserPresence>>>& List: PresenceLists)
		{
			if(TSharedRef<FUserPresence>* Presence = List.Value.Find(PresenceAccountId))
			{
				OutObservers.Add(List.Key);
				OutEntries.Add(*Presence);
			}
		}
	}

	TOnlineAsyncOpHandle<FQueryPresence> FPresenceXbl::QueryPresence(FQueryPresence::Params&& InParams)
	{
		TOnlineAsyncOpRef<FQueryPresence> Op = GetJoinableOp<FQueryPresence>(MoveTemp(InParams));
		if (!Op->IsReady())
		{
			const FQueryPresence::Params& Params = Op->GetParams();
			if (!Params.LocalAccountId.IsValid())
			{
				Op->SetError(Errors::InvalidParams());
				return Op->GetHandle();
			}

			Op->Then([this](TOnlineAsyncOp<FQueryPresence>& Op)
				{
					TSharedRef<TPromise<void>> Promise = MakeShared<TPromise<void>>();
					TFuture<void> Future = Promise->GetFuture();

					TSharedRef<FGDKAsyncBlock> AsyncBlock = MakeShared<FGDKAsyncBlock>(nullptr, [Promise](class FGDKAsyncBlock*)
						{
							Promise->EmplaceValue();
						});
					Op.Data.Set<TSharedRef<FGDKAsyncBlock>>(UE_XBL_ASYNC_BLOCK_KEY_NAME, AsyncBlock);

					const FQueryPresence::Params& Params = Op.GetParams();


					uint64 XUID = FOnlineAccountIdRegistryXbl::Get().Find(Params.TargetAccountId );
					if (XUID == 0)
					{
						Op.SetError(Errors::InvalidUser());
						Promise->EmplaceValue();
						UE_LOGF(LogOnlineServices, Warning, "[%s]: No XUID for target user.", __FUNCTION__);
					}

					FGDKContextHandle GDKContext = static_cast<FOnlineServicesXbl&>(Services).ContextManager->GetGDKContext(FOnlineAccountIdRegistryXbl::Get().Find(Params.LocalAccountId));

					if(Params.bListenToChanges)
					{
						XblPresenceTrackUsers(GDKContext, &XUID,1);
					}
					else
					{
						XblPresenceStopTrackingUsers(GDKContext, &XUID, 1);
					}
					HRESULT Result = XblPresenceGetPresenceAsync(GDKContext, XUID, *AsyncBlock);
					if (FAILED(Result))
					{
						FOnlineError Error = Errors::FromHRESULT(Result);
						Op.SetError(MoveTemp(Error));
						Promise->EmplaceValue();
						UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to request presence. Error %ls", __FUNCTION__, *Error.GetLogString());
					}

					return Future;
				}).Then([this](TOnlineAsyncOp<FQueryPresence>& Op)
					{
						const TSharedRef<FGDKAsyncBlock>& AsyncBlock = GetOpDataChecked<TSharedRef<FGDKAsyncBlock>>(Op, UE_XBL_ASYNC_BLOCK_KEY_NAME);
	

						XblPresenceRecordHandle PresenceRecordHandle;
						HRESULT Result = XblPresenceGetPresenceResult(*AsyncBlock, &PresenceRecordHandle);
						if (FAILED(Result))
						{
							FOnlineError Error = Errors::FromHRESULT(Result);
							Op.SetError(MoveTemp(Error));
							UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to get presence records. Error %ls", __FUNCTION__, *Error.GetLogString());
							return;
						}
						
						uint64 UserXuid = 0;
						XblPresenceUserState UserOnlineStatus;
						uint64 NumDeviceRecords = 0;
						const XblPresenceDeviceRecord* UserDeviceRecords;

						HRESULT IdResult = XblPresenceRecordGetXuid(PresenceRecordHandle, &UserXuid);
						HRESULT StatusResult = XblPresenceRecordGetUserState(PresenceRecordHandle, &UserOnlineStatus);
						HRESULT DeviceResult = XblPresenceRecordGetDeviceRecords(PresenceRecordHandle, &UserDeviceRecords, &NumDeviceRecords);

						if (FAILED(IdResult))
						{
							FOnlineError Error = Errors::FromHRESULT(IdResult);
							Op.SetError(MoveTemp(Error));
							UE_LOGF(LogOnlineServices, Verbose, "[%s]: Failed to parse presence record. Error %ls", __FUNCTION__, *Error.GetLogString());
							return;
						}
						if (FAILED(StatusResult))
						{
							FOnlineError Error = Errors::FromHRESULT(StatusResult);
							Op.SetError(MoveTemp(Error));
							UE_LOGF(LogOnlineServices, Verbose, "[%s]: Failed to parse presence record. Error %ls", __FUNCTION__, *Error.GetLogString());
							return;
						}
						if (FAILED(DeviceResult))
						{
							FOnlineError Error = Errors::FromHRESULT(DeviceResult);
							Op.SetError(MoveTemp(Error));
							UE_LOGF(LogOnlineServices, Verbose, "[%s]: Failed to parse presence record. Error %ls", __FUNCTION__, *Error.GetLogString());
							return;
						}

						FAccountId UserAccountId = FOnlineAccountIdRegistryXbl::Get().Find(UserXuid);
						if (!UserAccountId.IsValid())
						{
							Op.SetError(Errors::InvalidUser());
							UE_LOGF(LogOnlineServices, Verbose, "[%s]: No local account ID register for returned presence.", __FUNCTION__);
						}
						const FQueryPresence::Params& Params = Op.GetParams();
						TSharedRef<FUserPresence> Presence = FindOrCreatePresence(Params.LocalAccountId, UserAccountId);
						Presence->Status = ToPresenceStatus(UserOnlineStatus);
						Presence->StatusString = LexToString(ToPresenceStatus(UserOnlineStatus));
						ParsePresenceFromDeviceRecords(Presence, UserDeviceRecords, NumDeviceRecords, TitleId);
						///** Session state */ CDATODO get multiplayer activities 
						//EUserPresenceJoinability Joinability = EUserPresenceJoinability::Unknown;
						///** Session keys */
						//FPresenceProperties Properties;					
						
						XblPresenceRecordCloseHandle(PresenceRecordHandle);
						

						Op.SetResult(FQueryPresence::Result{ MoveTemp(Presence) });
					});

				Op->Enqueue(Services.GetParallelQueue());
		}
		return Op->GetHandle();
	}

	TOnlineAsyncOpHandle<FBatchQueryPresence> FPresenceXbl::BatchQueryPresence(FBatchQueryPresence::Params&& InParams)
	{
		TOnlineAsyncOpRef<FBatchQueryPresence> Op = GetJoinableOp<FBatchQueryPresence>(MoveTemp(InParams));
		if (!Op->IsReady())
		{
			const FBatchQueryPresence::Params& Params = Op->GetParams();
			if (Params.TargetAccountIds.IsEmpty())
			{
				Op->SetError(Errors::InvalidParams());
				return Op->GetHandle();
			}

			Op->Then([this](TOnlineAsyncOp<FBatchQueryPresence>& Op)
				{
					TSharedRef<TPromise<void>> Promise = MakeShared<TPromise<void>>();
					TFuture<void> Future = Promise->GetFuture();

					TSharedRef<FGDKAsyncBlock> AsyncBlock = MakeShared<FGDKAsyncBlock>(nullptr, [Promise](class FGDKAsyncBlock*)
						{
							Promise->EmplaceValue();
						});
					Op.Data.Set<TSharedRef<FGDKAsyncBlock>>(UE_XBL_ASYNC_BLOCK_KEY_NAME, AsyncBlock);

					const FBatchQueryPresence::Params& Params = Op.GetParams();

					TArray<uint64> XUIDs;
					for (const FAccountId& TargetAccountId : Params.TargetAccountIds)
					{
						if (uint64 XUID = FOnlineAccountIdRegistryXbl::Get().Find(TargetAccountId))
						{
							XUIDs.Add(XUID);
						}
						else
						{
							UE_LOGF(LogOnlineServices, Verbose, "[%s]: No XUIDs for target users  %ls", __FUNCTION__, *ToString(TargetAccountId));
						}
					}

					if (XUIDs.IsEmpty())
					{
						Op.SetError(Errors::InvalidUser());
						Promise->EmplaceValue();
						UE_LOGF(LogOnlineServices, Warning, "[%s]: No XUIDs for target users.", __FUNCTION__);
					}

					FGDKContextHandle GDKContext = static_cast<FOnlineServicesXbl&>(Services).ContextManager->GetGDKContext(FOnlineAccountIdRegistryXbl::Get().Find(Params.LocalAccountId));
					XblPresenceQueryFilters Filter{};
					Filter.detailLevel = XblPresenceDetailLevel::All;

					if (Params.bListenToChanges)
					{
						XblPresenceTrackUsers(GDKContext, XUIDs.GetData(), XUIDs.Num());
					}
					else
					{
						XblPresenceStopTrackingUsers(GDKContext, XUIDs.GetData(), XUIDs.Num());
					}
					HRESULT Result = XblPresenceGetPresenceForMultipleUsersAsync(GDKContext, XUIDs.GetData(), XUIDs.Num(), &Filter, *AsyncBlock);
					if (FAILED(Result))
					{
						FOnlineError Error = Errors::FromHRESULT(Result);
						Op.SetError(MoveTemp(Error));
						Promise->EmplaceValue();
						UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to request presence. Error %ls", __FUNCTION__, *Error.GetLogString());
					}

					return Future;
				}).Then([this](TOnlineAsyncOp<FBatchQueryPresence>& Op)
					{
						const TSharedRef<FGDKAsyncBlock>& AsyncBlock = GetOpDataChecked<TSharedRef<FGDKAsyncBlock>>(Op, UE_XBL_ASYNC_BLOCK_KEY_NAME);
						size_t NumResults = 0;
						HRESULT Result = XblPresenceGetPresenceForMultipleUsersResultCount(*AsyncBlock, &NumResults);
						if (FAILED(Result))
						{
							FOnlineError Error = Errors::FromHRESULT(Result);
							Op.SetError(MoveTemp(Error));
							UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to get presence record count. Error %ls", __FUNCTION__, *Error.GetLogString());
							return;
						}

						XblPresenceRecordHandle* PresenceRecordHandles = new XblPresenceRecordHandle[NumResults];
						Result = XblPresenceGetPresenceForMultipleUsersResult(*AsyncBlock, PresenceRecordHandles, NumResults);
						if (FAILED(Result))
						{
							FOnlineError Error = Errors::FromHRESULT(Result);
							Op.SetError(MoveTemp(Error));
							UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to get presence records. Error %ls", __FUNCTION__, *Error.GetLogString());
							return;
						}
						FBatchQueryPresence::Result BatchResult;

						for (int i = 0; i < NumResults; ++i)
						{	
							uint64 UserXuid = 0;
							XblPresenceUserState UserOnlineStatus;
							uint64 NumDeviceRecords = 0;
							const XblPresenceDeviceRecord* UserDeviceRecords;

							HRESULT IdResult = XblPresenceRecordGetXuid(PresenceRecordHandles[i], &UserXuid);
							HRESULT StatusResult = XblPresenceRecordGetUserState(PresenceRecordHandles[i], &UserOnlineStatus);
							HRESULT DeviceResult = XblPresenceRecordGetDeviceRecords(PresenceRecordHandles[i], &UserDeviceRecords, &NumDeviceRecords);

							if (FAILED(IdResult) || FAILED(StatusResult) || FAILED(DeviceResult))
							{
								UE_LOGF(LogOnlineServices, Verbose, "[%s]: Failed to parse presence record.", __FUNCTION__);
								continue;
							}

							const FBatchQueryPresence::Params& Params = Op.GetParams();
							FAccountId UserAccountId = FOnlineAccountIdRegistryXbl::Get().Find(UserXuid);
							if (!UserAccountId.IsValid())
							{
								UE_LOGF(LogOnlineServices, Verbose, "[%s]: No local account ID register for returned presence.", __FUNCTION__);
								continue;
							}
							TSharedRef<FUserPresence> Presence = FindOrCreatePresence(Params.LocalAccountId, UserAccountId);
							BatchResult.Presences.Emplace(Presence);
							Presence->Status = ToPresenceStatus(UserOnlineStatus);
							Presence->StatusString = LexToString(ToPresenceStatus(UserOnlineStatus));
							ParsePresenceFromDeviceRecords(Presence, UserDeviceRecords, NumDeviceRecords, TitleId);
							///** Session state */ CDATODO get multiplayer activities ?
							//EUserPresenceJoinability Joinability = EUserPresenceJoinability::Unknown;
							///** Session keys */
							//FPresenceProperties Properties;

						}
						for (int i = 0; i < NumResults; ++i)
						{
							XblPresenceRecordCloseHandle(PresenceRecordHandles[i]);
						}

						Op.SetResult(MoveTemp(BatchResult));
					});

				Op->Enqueue(Services.GetParallelQueue());
		}
		return Op->GetHandle();
	}

	TOnlineResult<FGetCachedPresence> FPresenceXbl::GetCachedPresence(FGetCachedPresence::Params&& Params)
	{
		if (TMap<FAccountId, TSharedRef<FUserPresence>>* PresenceList = PresenceLists.Find(Params.LocalAccountId))
		{
			TSharedRef<FUserPresence>* PresencePtr = PresenceList->Find(Params.TargetAccountId);
			if (PresencePtr)
			{
				FGetCachedPresence::Result Result = { *PresencePtr };
				return TOnlineResult<FGetCachedPresence>(MoveTemp(Result));
			}
		}
		return TOnlineResult<FGetCachedPresence>(Errors::NotFound());
	}

	TValueOrError<FString, UE::Online::FOnlineError> PresenceProperty_To_XBL_Presence_String(const FPresenceProperty& PresenceAttribute)
	{
		FString UpdatedPropertyValueStr;

		if (PresenceAttribute.IsType<FPresencePropertiesRef>())
		{
			FPresencePropertiesRef UpdatedPropertyValueRef = PresenceAttribute.Get<FPresencePropertiesRef>();

			UpdatedPropertyValueStr = FString::Printf(TEXT("%s"), *NestedVariantToJson(UpdatedPropertyValueRef.ToSharedPtr()));
		}
		else if (PresenceAttribute.IsType<bool>())
		{
			UpdatedPropertyValueStr = FString::Printf(TEXT("%s"), *ToLogString(PresenceAttribute.Get<bool>()));
		}
		else if (PresenceAttribute.IsType<int64>())
		{
			UpdatedPropertyValueStr = FString::Printf(TEXT("%lld"), PresenceAttribute.Get<int64>());
		}
		else if (PresenceAttribute.IsType<double>())
		{
			UpdatedPropertyValueStr = FString::Printf(TEXT("%f"), PresenceAttribute.Get<double>());
		}
		else if (PresenceAttribute.IsType<FString>())
		{
			UpdatedPropertyValueStr = FString::Printf(TEXT("%s"), *PresenceAttribute.Get<FString>());
		}
		else
		{
			return MakeError(Errors::CantParse());
		}

		return MakeValue(MoveTemp(UpdatedPropertyValueStr));
	}

	TOnlineAsyncOpHandle<FUpdatePresence> FPresenceXbl::UpdatePresence(FUpdatePresence::Params&& InParams)
	{
		// The only presence the client sets manually on GDK platforms is the Rich presence and session presence.
		// Rich presence is not set directly as a string, but and an ID string and an array of values. 
		// The XboxLive service looks up a display string based on the ID and substitutes the values in place to generate the rich presence result.
		// We take the local RichPresenceString as the ID and look for values in the PresenceProprties with that ID as their key with an index.
		TOnlineAsyncOpRef<FUpdatePresence> Op = GetOp<FUpdatePresence>(MoveTemp(InParams));
		const FUpdatePresence::Params& Params = Op->GetParams();
		if (!Services.Get<FAuthXbl>()->IsLoggedIn(Params.LocalAccountId))
		{
			Op->SetError(Errors::NotLoggedIn());
			return Op->GetHandle();
		}

		Op->Then([this](TOnlineAsyncOp<FUpdatePresence>& Op) 
			{
				const FUpdatePresence::Params& Params = Op.GetParams();
				TSharedRef<TPromise<void>> Promise = MakeShared<TPromise<void>>();
				TFuture<void> Future = Promise->GetFuture();
				TSharedRef<FGDKAsyncBlock> AsyncBlock = MakeShared<FGDKAsyncBlock>(nullptr, [Promise](class FGDKAsyncBlock*)
					{
						Promise->EmplaceValue();
					});
				Op.Data.Set<TSharedRef<FGDKAsyncBlock>>(UE_XBL_ASYNC_BLOCK_KEY_NAME, AsyncBlock);


				const ANSICHAR* Scid = nullptr;
				XblGetScid(&Scid);
				XblPresenceRichPresenceIds PresenceIds;
				FMemory::Memcpy(PresenceIds.scid, Scid, XBL_SCID_LENGTH);
				TArray<ANSICHAR*> PresenceTokens;
				TArray<TArray<ANSICHAR>> PresenceStringStorage;
				
				PresenceStringStorage.AddDefaulted(Params.Presence->Properties.Num());

				uint32 Idx = 0;
				for (const TPair<FPresenceProperties::KeyType, FPresenceProperty>& Property : Params.Presence->Properties)
				{
					if (Property.Key.Find(Params.Presence->RichPresenceString) != 0)
					{
						continue;
					}

					TValueOrError<FString, FOnlineError> PropertyValueResult = PresenceProperty_To_XBL_Presence_String(Property.Value);
					if (PropertyValueResult.HasValue())
					{
						const FString& ValueString = PropertyValueResult.GetValue();
						const FTCHARToUTF8 BufferString(*ValueString);
						int32 BufferStringLen = BufferString.Length();

						TArray<ANSICHAR>& CharBuffer = PresenceStringStorage[Idx];
						CharBuffer.Append(BufferString.Get(), BufferStringLen + 1);
						PresenceTokens.Add(CharBuffer.GetData());
						++Idx;
					}
					else
					{
						FOnlineError Error = PropertyValueResult.GetError();
						Op.SetError(MoveTemp(Error));
						Promise->EmplaceValue();
						UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to set presence. Can't parse presence property %ls. Error %ls", __FUNCTION__, *Property.Key, *Error.GetLogString());
						return Future;
					}
				}				

				if (PresenceTokens.Num() > 0)
				{
					PresenceIds.presenceTokenIds = const_cast<const ANSICHAR**>(PresenceTokens.GetData());
					PresenceIds.presenceTokenIdsCount = PresenceTokens.Num();
				}
				else
				{
					PresenceIds.presenceTokenIds = nullptr;
					PresenceIds.presenceTokenIdsCount = 0;
				}

				const FTCHARToUTF8 BufferString(*Params.Presence->RichPresenceString);
				PresenceIds.presenceId = BufferString.Get();

				FGDKContextHandle GDKContext = static_cast<FOnlineServicesXbl&>(Services).ContextManager->GetGDKContext(FOnlineAccountIdRegistryXbl::Get().Find(Params.LocalAccountId));


				HRESULT Result = XblPresenceSetPresenceAsync(GDKContext, true, &PresenceIds, *AsyncBlock);

				if (FAILED(Result))
				{
					FOnlineError Error = Errors::FromHRESULT(Result);
					Op.SetError(MoveTemp(Error));
					Promise->EmplaceValue();
					UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to set presence. Error %ls", __FUNCTION__, *Error.GetLogString());
				}	

				return Future;
			})
			.Then([this](TOnlineAsyncOp<FUpdatePresence>& Op)
				{
					const TSharedRef<FGDKAsyncBlock>& AsyncBlock = GetOpDataChecked<TSharedRef<FGDKAsyncBlock>>(Op, UE_XBL_ASYNC_BLOCK_KEY_NAME);

					if (FAILED(AsyncBlock->GetStatus()))
					{
						FOnlineError Error = Errors::FromHRESULT(AsyncBlock->GetStatus());
						Op.SetError(MoveTemp(Error));
						UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to set presence. Error %ls", __FUNCTION__, *Error.GetLogString());
						return;
					}

					Op.SetResult(FUpdatePresence::Result{ });
				})
				.Enqueue(GetSerialQueue());

				return Op->GetHandle();
	}

	TOnlineAsyncOpHandle<FPartialUpdatePresence> FPresenceXbl::PartialUpdatePresence(FPartialUpdatePresence::Params&& InParams)
	{
		// The only presence the client sets manually on GDK platforms is the Rich presence and session presence.
		// Rich presence is not set directly as a string, but and an ID string and an array of values. 
		// The XboxLive service looks up a display string based on the ID and substitutes the values in place to generate the rich presence result.
		// We take the local RichPresenceString as the ID and look for values in the PresenceProprties with that ID as their key with an index.
		TOnlineAsyncOpRef<FPartialUpdatePresence> Op = GetOp<FPartialUpdatePresence>(MoveTemp(InParams));
		const FPartialUpdatePresence::Params& Params = Op->GetParams();
		if (!Services.Get<FAuthXbl>()->IsLoggedIn(Params.LocalAccountId))
		{
			Op->SetError(Errors::NotLoggedIn());
			return Op->GetHandle();
		}

		if(!Params.Mutations.RichPresenceString.IsSet())
		{
			Op->SetResult(FPartialUpdatePresence::Result{ });
			return Op->GetHandle();
		}

		Op->Then([this](TOnlineAsyncOp<FPartialUpdatePresence>& Op)
			{
				const FPartialUpdatePresence::Params& Params = Op.GetParams();
				TSharedRef<TPromise<void>> Promise = MakeShared<TPromise<void>>();
				TFuture<void> Future = Promise->GetFuture();
				TSharedRef<FGDKAsyncBlock> AsyncBlock = MakeShared<FGDKAsyncBlock>(nullptr, [Promise](class FGDKAsyncBlock*)
					{
						Promise->EmplaceValue();
					});
				Op.Data.Set<TSharedRef<FGDKAsyncBlock>>(UE_XBL_ASYNC_BLOCK_KEY_NAME, AsyncBlock);


				const ANSICHAR* Scid = nullptr;
				XblGetScid(&Scid);
				XblPresenceRichPresenceIds PresenceIds;
				FMemory::Memcpy(PresenceIds.scid, Scid, XBL_SCID_LENGTH);
				TArray<ANSICHAR*> PresenceTokens;
				TArray<TArray<ANSICHAR>> PresenceStringStorage;

				PresenceStringStorage.AddDefaulted(Params.Mutations.UpdatedProperties.Num());

				uint32 Idx = 0;
				for (const TPair<FPresenceProperties::KeyType, FPresenceProperty>& Property : Params.Mutations.UpdatedProperties)
				{
					if (Params.Mutations.RichPresenceString.IsSet() && Property.Key.Find(Params.Mutations.RichPresenceString.GetValue()) != 0)
					{
						continue;
					}

					TValueOrError<FString, FOnlineError> PropertyValueResult = PresenceProperty_To_XBL_Presence_String(Property.Value);
					if (PropertyValueResult.HasValue())
					{
						const FString& ValueString = PropertyValueResult.GetValue();
						const FTCHARToUTF8 BufferString(*ValueString);
						int32 BufferStringLen = BufferString.Length();

						TArray<ANSICHAR>& CharBuffer = PresenceStringStorage[Idx];
						CharBuffer.Append(BufferString.Get(), BufferStringLen + 1);
						PresenceTokens.Add(CharBuffer.GetData());
						++Idx;
					}
					else
					{
						FOnlineError Error = PropertyValueResult.GetError();
						Op.SetError(MoveTemp(Error));
						Promise->EmplaceValue();
						UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to set presence. Can't parse presence property %ls. Error %ls", __FUNCTION__, *Property.Key, *Error.GetLogString());
						return Future;
					}
				}

				if (PresenceTokens.Num() > 0)
				{
					PresenceIds.presenceTokenIds = const_cast<const ANSICHAR**>(PresenceTokens.GetData());
					PresenceIds.presenceTokenIdsCount = PresenceTokens.Num();
				}
				else
				{
					PresenceIds.presenceTokenIds = nullptr;
					PresenceIds.presenceTokenIdsCount = 0;
				}
				const FTCHARToUTF8 BufferString(*Params.Mutations.RichPresenceString.GetValue());
				PresenceIds.presenceId = BufferString.Get();

				FGDKContextHandle GDKContext = static_cast<FOnlineServicesXbl&>(Services).ContextManager->GetGDKContext(FOnlineAccountIdRegistryXbl::Get().Find(Params.LocalAccountId));


				HRESULT Result = XblPresenceSetPresenceAsync(GDKContext, true, &PresenceIds, *AsyncBlock);

				if (FAILED(Result))
				{
					FOnlineError Error = Errors::FromHRESULT(Result);
					Op.SetError(MoveTemp(Error));
					Promise->EmplaceValue();
					UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to set presence. Error %ls", __FUNCTION__, *Error.GetLogString());
				}

				return Future;

			})
			.Then([this](TOnlineAsyncOp<FPartialUpdatePresence>& Op)
				{
					const TSharedRef<FGDKAsyncBlock>& AsyncBlock = GetOpDataChecked<TSharedRef<FGDKAsyncBlock>>(Op, UE_XBL_ASYNC_BLOCK_KEY_NAME);

					if (FAILED(AsyncBlock->GetStatus()))
					{
						FOnlineError Error = Errors::FromHRESULT(AsyncBlock->GetStatus());
						Op.SetError(MoveTemp(Error));
						UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to set presence. Error %ls", __FUNCTION__, *Error.GetLogString());
						return;
					}

					Op.SetResult(FPartialUpdatePresence::Result{ });
				})
				.Enqueue(GetSerialQueue());

				return Op->GetHandle();
	}	

	void FPresenceXbl::OnlineStatusUpdate(const FOnlineStatusUpdate& Update)
	{
		FAccountId TargetAccount = FOnlineAccountIdRegistryXbl::Get().Find(Update.XUID);
		if(TargetAccount.IsValid())
		{
			TArray<FAccountId> Observers;
			TArray<TSharedRef<FUserPresence>> Entries;
			FindPresenceEntriesAndObservingLocalUsers(TargetAccount, Entries, Observers);
			for (int i = 0; i < Entries.Num();++i)
			{
				EUserPresenceStatus OldStatus = Entries[i]->Status;
				Entries[i]->Status = Update.bOnline ? EUserPresenceStatus::Online : EUserPresenceStatus::Offline;
				if (Entries[i]->Status != OldStatus)
				{
					FPresenceUpdated PresenceUpdatedParams = { Observers[i], Entries[i] };
					OnPresenceUpdatedEvent.Broadcast(PresenceUpdatedParams);
				}
			}
		
		}
	}

	void FPresenceXbl::TitleStatusUpdate(const FTitleStatusUpdate& Update)
	{
		if(TitleId != Update.TitleId)
		{
			return;
		}

		FAccountId TargetAccount = FOnlineAccountIdRegistryXbl::Get().Find(Update.XUID);
		if (TargetAccount.IsValid())
		{
			TArray<FAccountId> Observers;
			TArray<TSharedRef<FUserPresence>> Entries;
			FindPresenceEntriesAndObservingLocalUsers(TargetAccount, Entries, Observers);
			for (int i = 0; i < Entries.Num(); ++i)
			{
				EUserPresenceGameStatus OldStatus = Entries[i]->GameStatus;
				switch(static_cast<XblPresenceTitleState>(Update.TitleState))
				{
				case XblPresenceTitleState::Started:
					Entries[i]->GameStatus = EUserPresenceGameStatus::PlayingThisGame;
					break;
				case XblPresenceTitleState::Ended:
					Entries[i]->GameStatus = EUserPresenceGameStatus::PlayingOtherGame;
					break;
				default:
					Entries[i]->GameStatus = EUserPresenceGameStatus::Unknown;
					break;
				}

				if (Entries[i]->GameStatus != EUserPresenceGameStatus::PlayingThisGame)
				{
					// Clear game related presence if we the target is not in out title anymore.
					Entries[i]->Joinability = EUserPresenceJoinability::Unknown;					
					Entries[i]->StatusString = FString();
					Entries[i]->RichPresenceString = FString();
					Entries[i]->Properties.Reset();
				}

				if (Entries[i]->GameStatus != OldStatus)
				{
					FPresenceUpdated PresenceUpdatedParams = { Observers[i], Entries[i] };
					OnPresenceUpdatedEvent.Broadcast(PresenceUpdatedParams);
				}
			}

		}
	}


} // namespace UE::Online

#endif // WITH_GRDK
