// Copyright Epic Games, Inc. All Rights Reserved.
#if WITH_GRDK

#include "Online/PrivilegesXbl.h"
#include "Online/AuthXbl.h"

#include "Algo/Find.h"
#include "Algo/ForEach.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Misc/ScopeRWLock.h"
#include "Online/OnlineServicesXbl.h"
#include "Online/OnlineUtils.h"
#include "Online/OnlineUtilsCommon.h"
#include "Online/Windows/WindowsOnlineErrorDefinitions.h"
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"
#include "Misc/CommandLine.h"

THIRD_PARTY_INCLUDES_START
#include <XStore.h>
THIRD_PARTY_INCLUDES_END

#define UE_PRIVILEGE_KEY_NAME  TEXT("Privilege")
#define UE_PATCHCHECK_KEY_NAME  TEXT("PatchCheck")
namespace UE::Online {




	FPrivilegesXbl::FPrivilegesXbl(FOnlineServicesXbl& InServices)
		: FPrivilegesCommon(InServices)
	{
	}

	void FPrivilegesXbl::Initialize()
	{
		FPrivilegesCommon::Initialize();
	}

	void FPrivilegesXbl::PreShutdown()
	{
		FPrivilegesCommon::PreShutdown();
	}

	bool ToXblPrivilege(EUserPrivileges InPrivilege, XUserPrivilege& OutPrivilege)
	{
		switch (InPrivilege)
		{
		case EUserPrivileges::CanPlayOnline:
		{
			OutPrivilege = XUserPrivilege::Multiplayer;
			break;
		}

		case EUserPrivileges::CanCommunicateViaTextOnline:
		case EUserPrivileges::CanCommunicateViaVoiceOnline:
		{
			OutPrivilege = XUserPrivilege::Communications;
			break;
		}

		case EUserPrivileges::CanUseUserGeneratedContent:
		{
			OutPrivilege = XUserPrivilege::UserGeneratedContent;
			break;
		}

		case EUserPrivileges::CanCrossPlay:
		{
			OutPrivilege = XUserPrivilege::CrossPlay;
			break;
		}

		default:
		{
			return false;
			
		}
		}
		return true;
	}

	TOnlineAsyncOpHandle<FQueryUserPrivilege> FPrivilegesXbl::QueryUserPrivilege(FQueryUserPrivilege::Params&& Params)
	{

		TOnlineAsyncOpRef<FQueryUserPrivilege> Op = GetJoinableOp<FQueryUserPrivilege>(MoveTemp(Params));
		if (!Op->IsReady())
		{

			Op->Then([this](TOnlineAsyncOp<FQueryUserPrivilege>& InAsyncOp)
				{// validate input, parse and store members in task. Begin patch check if applicable
					
					const FQueryUserPrivilege::Params& Params = InAsyncOp.GetParams();
					TSharedRef<TPromise<void>> Promise = MakeShared<TPromise<void>>();
					TFuture<void> Future = Promise->GetFuture();

					if (Params.Privilege == EUserPrivileges::CanPlay)
					{	// No platform equivalent, return true.
						InAsyncOp.SetResult(FQueryUserPrivilege::Result{ EPrivilegeResults::NoFailures });
						Promise->EmplaceValue();
						return Future;
					}

					TSharedRef<FGDKAsyncBlock> AsyncBlock = MakeShared<FGDKAsyncBlock>(nullptr, [Promise](class FGDKAsyncBlock*)
						{
							Promise->EmplaceValue();
						});

					// Capture async block on operation.
					InAsyncOp.Data.Set<TSharedRef<FGDKAsyncBlock>>(UE_XBL_ASYNC_BLOCK_KEY_NAME, AsyncBlock);

					FAuthGetLocalOnlineUserByOnlineAccountId::Params AuthParams;
					AuthParams.LocalAccountId = Params.LocalAccountId;
					TOnlineResult<UE::Online::FAuthGetLocalOnlineUserByOnlineAccountId> GetAccountResult = Services.Get<FAuthXbl>()->GetLocalOnlineUserByOnlineAccountId(MoveTemp(AuthParams));
					
					if (!GetAccountResult.IsOk())
					{
						InAsyncOp.SetError(Errors::InvalidParams());
						Promise->EmplaceValue();
						UE_LOGF(LogOnlineServices, Error, "[%s]: Failed with rrror %ls", __FUNCTION__, *Errors::InvalidParams().GetLogString());
						return Future;
					}	

					// Store account info on operation.
					InAsyncOp.Data.Set<TSharedRef<FAccountInfo>>(UE_XBL_ACCOUNT_INFO_KEY_NAME, GetAccountResult.GetOkValue().AccountInfo);

					XUserPrivilege PrivilegeQuery = XUserPrivilege::Multiplayer; 
					if (!ToXblPrivilege(Params.Privilege, PrivilegeQuery))
					{
						InAsyncOp.SetError(Errors::InvalidParams());
						Promise->EmplaceValue();
						return Future;
					}	

					bool bPatchcheck = (Params.Privilege == EUserPrivileges::CanCrossPlay || Params.Privilege == EUserPrivileges::CanPlayOnline);

					// Store platform privilege type and patch check flagon operation.
					InAsyncOp.Data.Set<XUserPrivilege>(UE_PRIVILEGE_KEY_NAME, PrivilegeQuery);
					InAsyncOp.Data.Set<bool>(UE_PATCHCHECK_KEY_NAME, bPatchcheck);

					if(bPatchcheck && !(FParse::Param(FCommandLine::Get(), TEXT("NoUpdateCheckGDK")) || static_cast<FOnlineServicesXbl&>(Services).GetOnlineEnvironment() == EOnlineEnvironment::Development))
					{
						XStoreContextHandle StoreContextHandle = nullptr;
						HRESULT Result = XStoreCreateContext(StaticCastSharedRef<FAccountInfoXbl>(GetAccountResult.GetOkValue().AccountInfo)->UserHandle, &StoreContextHandle);
						if (FAILED(Result) || StoreContextHandle == nullptr)
						{							
							FOnlineError Error = Errors::FromHRESULT(Result);
							UE_LOGF(LogOnlineServices, Error, "[%s]: Failed to create store context. Error %ls", __FUNCTION__, *Error.GetLogString());
							Promise->EmplaceValue();
							InAsyncOp.Data.Set<bool>(UE_PATCHCHECK_KEY_NAME, false);
							return Future;
						}	

						Result = XStoreQueryGameAndDlcPackageUpdatesAsync(StoreContextHandle, *AsyncBlock);
						if (FAILED(Result))
						{
							FOnlineError Error = Errors::FromHRESULT(Result);
							UE_LOGF(LogOnlineServices, Error, "[%s]: Failed to query packages. Error %ls", __FUNCTION__, *Error.GetLogString());
							Promise->EmplaceValue();
							InAsyncOp.Data.Set<bool>(UE_PATCHCHECK_KEY_NAME, false);
							return Future;
						}
						return Future;
					}
					Promise->EmplaceValue();
					return Future;

				}, FOnlineAsyncExecutionPolicy::RunOnThreadPool()).Then([this](TOnlineAsyncOp<FQueryUserPrivilege>& InAsyncOp)
					{// Process patch check result if applicable
						
						bool bPatchcheck = GetOpDataChecked<bool>(InAsyncOp, UE_PATCHCHECK_KEY_NAME);

						if (!bPatchcheck)
						{
							return;
						}

						const TSharedRef<FGDKAsyncBlock>& AsyncBlock = GetOpDataChecked<TSharedRef<FGDKAsyncBlock>>(InAsyncOp, UE_XBL_ASYNC_BLOCK_KEY_NAME);


						uint32 ResultCount = 0;
						HRESULT Result = XStoreQueryGameAndDlcPackageUpdatesResultCount(*AsyncBlock, &ResultCount);
						if (Result == S_OK)
						{
							TArray<XStorePackageUpdate> PackageArray;
							if (ResultCount > 0)
							{
								PackageArray.Reserve(ResultCount);
								Result = XStoreQueryGameAndDlcPackageUpdatesResult(*AsyncBlock, ResultCount, PackageArray.GetData());
								PackageArray.SetNum(ResultCount);
								if (PackageArray[0].isMandatory)
								{
									UE_LOGF(LogOnlineServices, Warning, "[%s]: Package update pending", __FUNCTION__);
									InAsyncOp.SetError(Errors::IncompatibleVersion());				
								}
							}
						}
						else
						{
							FOnlineError Error = Errors::FromHRESULT(Result);
							UE_LOGF(LogOnlineServices, Warning, "[%s]: Query packages. Error %ls ", __FUNCTION__, *Error.GetLogString());
						}						
					}).Then([this](TOnlineAsyncOp<FQueryUserPrivilege>& InAsyncOp)
					{ // preform privilege check and return results. Launching and waiting for platform UI if needed.
						const FQueryUserPrivilege::Params& Params = InAsyncOp.GetParams();
						const TSharedRef<FAccountInfo>& AccountInfo = GetOpDataChecked<TSharedRef<FAccountInfo>>(InAsyncOp, UE_XBL_ACCOUNT_INFO_KEY_NAME);
						const XUserPrivilege PrivilegeQuery = GetOpDataChecked<XUserPrivilege>(InAsyncOp, UE_PRIVILEGE_KEY_NAME);

						TSharedRef<TPromise<void>> Promise = MakeShared<TPromise<void>>();
						TFuture<void> Future = Promise->GetFuture();

						TSharedRef<FGDKAsyncBlock> AsyncBlock = MakeShared<FGDKAsyncBlock>(nullptr, [Promise](class FGDKAsyncBlock*)
							{
								Promise->EmplaceValue();
							});

						// Capture async block on operation.
						InAsyncOp.Data.Set<TSharedRef<FGDKAsyncBlock>>(UE_XBL_ASYNC_BLOCK_KEY_NAME, AsyncBlock);

						bool bHasPrivilege = false;
						XUserPrivilegeDenyReason DenyReason = XUserPrivilegeDenyReason::None;
						FGDKUserHandle UserHandle = IGDKRuntimeModule::Get().GetUserHandleByPlatformId(AccountInfo->PlatformUserId);
						if(!UserHandle.IsValid())
						{
							UE_LOGF(LogOnlineServices, Warning, "[%s]:Failed to get user handle", __FUNCTION__);
							InAsyncOp.SetError(Errors::InvalidUser());
							return Future;
						}
						HRESULT Result = XUserCheckPrivilege(UserHandle, XUserPrivilegeOptions::None, PrivilegeQuery, &bHasPrivilege, &DenyReason);
						if(FAILED(Result) && Result != E_GAMEUSER_RESOLVE_USER_ISSUE_REQUIRED)
						{
							
							FOnlineError Error = Errors::FromHRESULT(Result);
							UE_LOGF(LogOnlineServices, Error, "[%s]: Failed to query packages. Error %ls", __FUNCTION__, *Error.GetLogString());
							InAsyncOp.SetError(MoveTemp(Error));
							Promise->EmplaceValue();
							return Future;
						}

						EPrivilegeResults PrivilegeResult = EPrivilegeResults::NoFailures;

						if(!bHasPrivilege)
						{
							if(Result == E_GAMEUSER_RESOLVE_USER_ISSUE_REQUIRED  && 
								(Params.ShowExternalUI == EShowPrivilegeResolutionUI::Show || Params.ShowExternalUI == EShowPrivilegeResolutionUI::Default))
							{
								Result = XUserResolvePrivilegeWithUiAsync(UserHandle, XUserPrivilegeOptions::None, PrivilegeQuery, *AsyncBlock);
								if (FAILED(Result))
								{
									FOnlineError Error = Errors::FromHRESULT(Result);
									InAsyncOp.SetError(MoveTemp(Error));
									Promise->EmplaceValue();
									UE_LOGF(LogOnlineServices, Error, "[%s]: Failed to launch external UI. Error %ls", __FUNCTION__, *Error.GetLogString());
									return Future;
								}
								return Future;
							}

							FPrivilegesXblConfig Config;
							TOnlineComponent::LoadConfig(Config);

							bool bXBLGoldRequired = Config.bXBLGoldRequired;
							
							if (XUserPrivilegeDenyReason::PurchaseRequired == DenyReason)
							{
								// If we allow non-premium accounts to play online multiplayer, override the result
								// of certain failures with success instead. Otherwise, indicate that the account
								// type was insufficient.
								if(bXBLGoldRequired)
								{
									PrivilegeResult = EPrivilegeResults::AccountTypeFailure;
								}
								else
								{									
									switch (Params.Privilege)
									{
									case EUserPrivileges::CanPlayOnline:
									case EUserPrivileges::CanCommunicateViaTextOnline:
									case EUserPrivileges::CanCommunicateViaVoiceOnline:
									case EUserPrivileges::CanCrossPlay:
										InAsyncOp.SetResult(FQueryUserPrivilege::Result{ PrivilegeResult });
										Promise->EmplaceValue();
										return Future;
									};
								}
							}	

							if (XUserPrivilegeDenyReason::Banned == DenyReason)
							{
								PrivilegeResult |= EPrivilegeResults::OnlinePlayRestricted;
							}

							switch (Params.Privilege)
							{
							case EUserPrivileges::CanPlayOnline:
								PrivilegeResult |= EPrivilegeResults::OnlinePlayRestricted;
								break;

							case EUserPrivileges::CanCommunicateViaTextOnline:
							case EUserPrivileges::CanCommunicateViaVoiceOnline:
								PrivilegeResult |= EPrivilegeResults::ChatRestriction;
								break;

							case EUserPrivileges::CanUseUserGeneratedContent:
								PrivilegeResult |= EPrivilegeResults::UGCRestriction;
								break;

							case EUserPrivileges::CanCrossPlay:
								PrivilegeResult |= EPrivilegeResults::OnlinePlayRestricted;
								break;

							default:
								break;
							}; // switch
						}					

						InAsyncOp.SetResult(FQueryUserPrivilege::Result{ PrivilegeResult });
						Promise->EmplaceValue();
						return Future;

						}).Then([this](TOnlineAsyncOp<FQueryUserPrivilege>& InAsyncOp)
						{
							const TSharedRef<FGDKAsyncBlock>& AsyncBlock = GetOpDataChecked<TSharedRef<FGDKAsyncBlock>>(InAsyncOp, UE_XBL_ASYNC_BLOCK_KEY_NAME);
							HRESULT Result = XUserResolvePrivilegeWithUiResult(*AsyncBlock);
							if (Result == S_OK)
							{
								InAsyncOp.SetResult(FQueryUserPrivilege::Result{ EPrivilegeResults::NoFailures });
							}
							else
							{
								FOnlineError Error = Errors::FromHRESULT(Result);
								UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to resolve privilege with external UI. Error %ls", __FUNCTION__, *Error.GetLogString());
								InAsyncOp.SetError(Errors::Cancelled());
							}
						}).Enqueue(GetSerialQueue());
	}

		return Op->GetHandle();	
	}

/* UE::Online */ }

#endif // WITH_GRDK
