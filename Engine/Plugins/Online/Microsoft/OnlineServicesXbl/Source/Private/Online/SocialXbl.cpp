// Copyright Epic Games, Inc. All Rights Reserved.
#if WITH_GRDK

#include "Online/SocialXbl.h"
#include "Online/AuthXbl.h"
#include "Online/UserInfoXbl.h"
#include "Online/OnlineBase.h"
#include "Online/OnlineServicesXbl.h"
#include "Online/OnlineUtils.h"
#include "Online/OnlineUtilsCommon.h"
#include "Online/Windows/WindowsOnlineErrorDefinitions.h"
THIRD_PARTY_INCLUDES_START
#include <xsapi-c/social_c.h>
#include <xsapi-c/privacy_c.h>
#include <XGameUI.h>
THIRD_PARTY_INCLUDES_END

#define UE_XBL_FRIEND_ID_KEY_NAME  TEXT("FriendIDs")
#define UE_XBL_RELATIONSHIPS_KEY_NAME  TEXT("RelationshipHandle")

namespace UE::Online { 



FSocialXbl::FSocialXbl(FOnlineServicesXbl& InServices)
	: FSocialCommon(InServices)
	, Services(InServices)
{

}

void FSocialXbl::Initialize()
{
	FSocialCommon::Initialize();
}

void FSocialXbl::PreShutdown()
{

}

void FSocialXbl::Tick(float DeltaSeconds)
{
}

TOnlineAsyncOpHandle<FQueryFriends> FSocialXbl::QueryFriends(FQueryFriends::Params&& InParams)
{
	TOnlineAsyncOpRef<FQueryFriends> Op = GetJoinableOp<FQueryFriends>(MoveTemp(InParams));
	if (Op->IsReady())
	{
		return Op->GetHandle();
	}

	Op->Then([this](TOnlineAsyncOp<FQueryFriends>& Op)
		{
			const FQueryFriends::Params& Params = Op.GetParams();

			TSharedRef<TPromise<void>> Promise = MakeShared<TPromise<void>>();
			TFuture<void> Future = Promise->GetFuture();

			TSharedRef<FGDKAsyncBlock> AsyncBlock = MakeShared<FGDKAsyncBlock>(nullptr, [Promise](class FGDKAsyncBlock*)
				{
					Promise->EmplaceValue();
				});
			// Capture async block on operation.
			Op.Data.Set<TSharedRef<FGDKAsyncBlock>>(UE_XBL_ASYNC_BLOCK_KEY_NAME, AsyncBlock);

			if (!Params.LocalAccountId.IsValid())
			{
				Op.SetError(Errors::InvalidParams());
				Promise->EmplaceValue();
				return Future;
			}

			if (!Services.Get<FAuthXbl>()->IsLoggedIn(Params.LocalAccountId))
			{
				Op.SetError(Errors::NotLoggedIn());
				Promise->EmplaceValue();
				return Future;
			}

			uint64 XUID = FOnlineAccountIdRegistryXbl::Get().Find(Params.LocalAccountId);
			if (XUID == 0)
			{
				Op.SetError(Errors::InvalidUser());
				Promise->EmplaceValue();
				return Future;
			}
			FGDKContextHandle GDKContext = static_cast<FOnlineServicesXbl&>(Services).ContextManager->GetGDKContext(XUID);
			if (!GDKContext.IsValid())
			{
				Op.SetError(Errors::InvalidUser());
				Promise->EmplaceValue();
				return Future;
			}

			HRESULT Result = XblSocialGetSocialRelationshipsAsync(GDKContext, XUID, XblSocialRelationshipFilter::All, 0, 0, *AsyncBlock);
			if (Result != S_OK)
			{
				FOnlineError Error = Errors::FromHRESULT(Result);
				UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to make query. Error %ls", __FUNCTION__, *Error.GetLogString());
				Op.SetError(MoveTemp(Error));
				Promise->EmplaceValue();
				return Future;
			}
			return Future;
		})
		.Then([this](TOnlineAsyncOp<FQueryFriends>& Op)
			{
				const TSharedRef<FGDKAsyncBlock>& AsyncBlock = GetOpDataChecked<TSharedRef<FGDKAsyncBlock>>(Op, UE_XBL_ASYNC_BLOCK_KEY_NAME);
				XblSocialRelationshipResultHandle RelationshipHandle = nullptr;
				HRESULT Result = XblSocialGetSocialRelationshipsResult(*AsyncBlock, &RelationshipHandle);
				if (FAILED(Result))
				{
					FOnlineError Error = Errors::FromHRESULT(Result);
					UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed query friends. Error %ls", __FUNCTION__, *Error.GetLogString());
					Op.SetError(MoveTemp(Error));
					return;
				}

				const XblSocialRelationship* SocialRelationships = nullptr;
				size_t ItemCount = 0;
				Result = XblSocialRelationshipResultGetRelationships(RelationshipHandle, &SocialRelationships, &ItemCount);
				if (Result != S_OK)
				{
					XblSocialRelationshipResultCloseHandle(RelationshipHandle);
					FOnlineError Error = Errors::FromHRESULT(Result);
					UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed get query friends result. Error %ls", __FUNCTION__, *Error.GetLogString());
					Op.SetError(MoveTemp(Error));
					return;
				}

				TSharedRef<TArray<uint64>> Friends = MakeShared<TArray<uint64>>();
				Op.Data.Set<TSharedRef<TArray<uint64>>>(UE_XBL_FRIEND_ID_KEY_NAME, Friends);

				for (int32 Index = 0; Index < ItemCount; ++Index)
				{
					(*Friends).Add(SocialRelationships[Index].xboxUserId);
				}
				Op.Data.Set<XblSocialRelationshipResultHandle>(UE_XBL_RELATIONSHIPS_KEY_NAME, RelationshipHandle);
			}).Then([this](TOnlineAsyncOp<FQueryFriends>& Op) mutable
				{
					XblSocialRelationshipResultHandle RelationshipHandle = GetOpDataChecked<XblSocialRelationshipResultHandle>(Op, UE_XBL_RELATIONSHIPS_KEY_NAME);
					TSharedRef<TPromise<TContinuationResult<void>>> Promise = MakeShared<TPromise<TContinuationResult<void>>>();
					TFuture<TContinuationResult<void>> Future = Promise->GetFuture();
					TSharedRef<FGDKAsyncBlock> AsyncBlock = MakeShared<FGDKAsyncBlock>(nullptr, [Promise, WeakOperation = TWeakPtr<TOnlineAsyncOp<FQueryFriends>>(Op.AsShared())](class FGDKAsyncBlock*) mutable
						{
							TSharedPtr<TOnlineAsyncOp<FQueryFriends>> Op = WeakOperation.Pin();
							if (Op)
							{
								XblSocialRelationshipResultHandle RelationshipHandle = GetOpDataChecked<XblSocialRelationshipResultHandle>(*Op, UE_XBL_RELATIONSHIPS_KEY_NAME);

								const XblSocialRelationship* SocialRelationships = nullptr;
								size_t ItemCount = 0;
								HRESULT Result = XblSocialRelationshipResultGetRelationships(RelationshipHandle, &SocialRelationships, &ItemCount);
								if (Result != S_OK)
								{
									XblSocialRelationshipResultCloseHandle(RelationshipHandle);
									FOnlineError Error = Errors::FromHRESULT(Result);
									UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed get query friends result. Error %ls", __FUNCTION__, *Error.GetLogString());
									Op->SetError(MoveTemp(Error));
									return;
								}
								const TSharedRef<TArray<uint64>>& Friends = GetOpDataChecked<TSharedRef<TArray<uint64>>>(*Op, UE_XBL_FRIEND_ID_KEY_NAME);

								for (int32 Index = 0; Index < ItemCount; ++Index)
								{
									(*Friends).Add(SocialRelationships[Index].xboxUserId);
								}
							}
							Promise->EmplaceValue(TContinuationResult<void>::Repeat());
						});
					// Capture async block on operation.
					Op.Data.Set<TSharedRef<FGDKAsyncBlock>>(UE_XBL_ASYNC_BLOCK_KEY_NAME, AsyncBlock);

					if(RelationshipHandle == nullptr)
					{
						Op.SetError(Errors::InvalidState());
						return Future;
					}
					bool bHasMore = false;
					HRESULT Result = XblSocialRelationshipResultHasNext(RelationshipHandle, &bHasMore);
					if (FAILED(Result))
					{
						XblSocialRelationshipResultCloseHandle(RelationshipHandle);
						FOnlineError Error = Errors::FromHRESULT(Result);
						UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed query friends pages. Error %ls", __FUNCTION__, *Error.GetLogString());
						Op.SetError(MoveTemp(Error));
						return Future;
					}
					if(!bHasMore)
					{
						XblSocialRelationshipResultCloseHandle(RelationshipHandle);
						Promise->EmplaceValue(TContinuationResult<void>::Complete());
						return Future;
					}

					Result = XblSocialRelationshipResultGetNextResult(*AsyncBlock, &RelationshipHandle);
					if (FAILED(Result))
					{
						XblSocialRelationshipResultCloseHandle(RelationshipHandle);
						FOnlineError Error = Errors::FromHRESULT(Result);
						UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed query friends page. Error %ls", __FUNCTION__, *Error.GetLogString());
						Op.SetError(MoveTemp(Error));
						return Future;
					}
					Op.Data.Set<XblSocialRelationshipResultHandle>(UE_XBL_RELATIONSHIPS_KEY_NAME, RelationshipHandle);
					return Future;

				}, FOnlineAsyncExecutionPolicy::RunOnThreadPool())
			.Then([this](TOnlineAsyncOp<FQueryFriends>& Op)
				{
					const TSharedRef<TArray<uint64>>& Friends = GetOpDataChecked<TSharedRef<TArray<uint64>>>(Op, UE_XBL_FRIEND_ID_KEY_NAME);
					const FQueryFriends::Params& Params = Op.GetParams();
					TMap<FAccountId, TSharedRef<FFriend>>& CurrenrFriendsList = FriendsLists.FindOrAdd(Params.LocalAccountId);
					TMap<FAccountId, TSharedRef<FFriend>> NewFriendsList;
					TArray<FAccountId>& AvoidList = AvoidLists.FindOrAdd(Params.LocalAccountId);

					for (uint64 XUID : *Friends)
					{						
						FAccountId FriendId = FOnlineAccountIdRegistryXbl::Get().FindOrAddAccountId(XUID);
						if (AvoidList.Contains(FriendId))
						{
							continue;
						}
						TSharedRef<FFriend> Friend = MakeShared<FFriend>();
						Friend->FriendId = FriendId;
						Friend->Relationship = ERelationship::Friend;
						NewFriendsList.Emplace(FriendId, Friend);
						if (!CurrenrFriendsList.Contains(FriendId))
						{
							BroadcastRelationshipUpdated(Params.LocalAccountId, FriendId, ERelationship::NotFriend, ERelationship::Friend);
						}						
					}
					FriendsLists.Emplace(Params.LocalAccountId,MoveTemp(NewFriendsList));

					Op.SetResult(FQueryFriends::Result{ });

				}, FOnlineAsyncExecutionPolicy::RunOnGameThread())
				.Enqueue(Services.GetParallelQueue());

		return Op->GetHandle();
}

TOnlineResult<FGetFriends> FSocialXbl::GetFriends(FGetFriends::Params&& Params)
{
	if (TMap<FAccountId, TSharedRef<FFriend>>* FriendsList = FriendsLists.Find(Params.LocalAccountId))
	{
		FGetFriends::Result Result;
		FriendsList->GenerateValueArray(Result.Friends);
		return TOnlineResult<FGetFriends>(MoveTemp(Result));
	}
	return TOnlineResult<FGetFriends>(Errors::InvalidState());
}

TOnlineAsyncOpHandle<FSendFriendInvite> FSocialXbl::SendFriendInvite(FSendFriendInvite::Params&& InParams)
{
	TOnlineAsyncOpRef<FSendFriendInvite> Op = GetJoinableOp<FSendFriendInvite>(MoveTemp(InParams));
	if (Op->IsReady())
	{
		return Op->GetHandle();
	}
	Op->Then([this](TOnlineAsyncOp<FSendFriendInvite>& Op)
		{
			const FSendFriendInvite::Params& Params = Op.GetParams();

			FShowUserProfile::Params CallParams;
			CallParams.LocalAccountId = Params.LocalAccountId;
			CallParams.AccountId = Params.TargetAccountId;
			TOnlineAsyncOpHandle<FShowUserProfile> QueryHandle = Services.Get<FUserInfoXbl>()->ShowUserProfile(MoveTemp(CallParams));

			TSharedRef<TPromise<void>> Promise = MakeShared<TPromise<void>>();
			TFuture<void> Future = Promise->GetFuture();
			QueryHandle.OnComplete([Promise](const TOnlineResult<FShowUserProfile>& Result)
				{
					Promise->EmplaceValue();
				});

		}).Then([this](TOnlineAsyncOp<FSendFriendInvite>& Op)
			{
				Op.SetResult(FSendFriendInvite::Result{});

			}).Enqueue(Services.GetParallelQueue());
	
	return Op->GetHandle();

}


TOnlineAsyncOpHandle<FQueryBlockedUsers> FSocialXbl::QueryBlockedUsers(FQueryBlockedUsers::Params&& InParams)
{
	TOnlineAsyncOpRef<FQueryBlockedUsers> Op = GetJoinableOp<FQueryBlockedUsers>(MoveTemp(InParams));
	if (Op->IsReady())
	{
		return Op->GetHandle();
	}

	Op->Then([this](TOnlineAsyncOp<FQueryBlockedUsers>& Op)
		{
			const FQueryBlockedUsers::Params& Params = Op.GetParams();

			TSharedRef<TPromise<void>> Promise = MakeShared<TPromise<void>>();
			TFuture<void> Future = Promise->GetFuture();

			TSharedRef<FGDKAsyncBlock> AsyncBlock = MakeShared<FGDKAsyncBlock>(nullptr, [Promise](class FGDKAsyncBlock*)
				{
					Promise->EmplaceValue();
				});
			// Capture async block on operation.
			Op.Data.Set<TSharedRef<FGDKAsyncBlock>>(UE_XBL_ASYNC_BLOCK_KEY_NAME, AsyncBlock);

			if (!Params.LocalAccountId.IsValid())
			{
				Op.SetError(Errors::InvalidParams());
				return;
			}

			if (!Services.Get<FAuthXbl>()->IsLoggedIn(Params.LocalAccountId))
			{
				Op.SetError(Errors::NotLoggedIn());
				return;
			}

			uint64 XUID = FOnlineAccountIdRegistryXbl::Get().Find(Params.LocalAccountId);
			if (XUID == 0)
			{
				Op.SetError(Errors::InvalidUser());
				return;
			}
			FGDKContextHandle GDKContext = static_cast<FOnlineServicesXbl&>(Services).ContextManager->GetGDKContext(XUID);
			if (!GDKContext.IsValid())
			{
				Op.SetError(Errors::InvalidUser());
				return;
			}

			HRESULT Result = XblPrivacyGetAvoidListAsync(GDKContext,*AsyncBlock);
			if (Result != S_OK)
			{
				FOnlineError Error = Errors::FromHRESULT(Result);
				UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to make query. Error %ls", __FUNCTION__, *Error.GetLogString());
				Op.SetError(MoveTemp(Error));
				return;
			}
		})
		.Then([this](TOnlineAsyncOp<FQueryBlockedUsers>& Op)
			{
				const TSharedRef<FGDKAsyncBlock>& AsyncBlock = GetOpDataChecked<TSharedRef<FGDKAsyncBlock>>(Op, UE_XBL_ASYNC_BLOCK_KEY_NAME);
				size_t ItemCount = 0;
				HRESULT Result = XblPrivacyGetAvoidListResultCount(*AsyncBlock, &ItemCount);
				if (FAILED(Result))
				{
					FOnlineError Error = Errors::FromHRESULT(Result);
					UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed query avoid list. Error %ls", __FUNCTION__, *Error.GetLogString());
					Op.SetError(MoveTemp(Error));
					return;
				}

				TArray<uint64> AvoidList;
				AvoidList.Reserve(ItemCount);
				Result = XblPrivacyGetAvoidListResult(*AsyncBlock, ItemCount, AvoidList.GetData());
				if (Result != S_OK)
				{
					FOnlineError Error = Errors::FromHRESULT(Result);
					UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed get query avoid list result. Error %ls", __FUNCTION__, *Error.GetLogString());
					Op.SetError(MoveTemp(Error));
					return;
				}
				const FQueryBlockedUsers::Params& Params = Op.GetParams();
				TArray<FAccountId>& CurrentAvoidList = AvoidLists.FindOrAdd(Params.LocalAccountId);
				TArray<FAccountId> NewAvoidList;
				TMap<FAccountId, TSharedRef<FFriend>>& FriendsList = FriendsLists.FindOrAdd(Params.LocalAccountId);

				for (uint64 XUID : AvoidList)
				{
					FAccountId AvoidId = FOnlineAccountIdRegistryXbl::Get().FindOrAddAccountId(XUID);
					NewAvoidList.Emplace(AvoidId);

					if (!CurrentAvoidList.Contains(AvoidId))
					{
						if (FriendsList.Contains(AvoidId))
						{
							FriendsList.Remove(AvoidId);
							BroadcastRelationshipUpdated(Params.LocalAccountId, AvoidId, ERelationship::Friend, ERelationship::Blocked);
						}
						else
						{
							BroadcastRelationshipUpdated(Params.LocalAccountId, AvoidId, ERelationship::NotFriend, ERelationship::Blocked);
						}
					}
				}
				AvoidLists.Emplace(Params.LocalAccountId, MoveTemp(NewAvoidList));
				Op.SetResult(FQueryBlockedUsers::Result{ });


				}, FOnlineAsyncExecutionPolicy::RunOnGameThread())
				.Enqueue(Services.GetParallelQueue());

				return Op->GetHandle();
}

TOnlineResult<FGetBlockedUsers> FSocialXbl::GetBlockedUsers(FGetBlockedUsers::Params&& Params)
{
	if (TArray<FAccountId>* AvoidList = AvoidLists.Find(Params.LocalAccountId))
	{
		FGetBlockedUsers::Result Result;
		Result.BlockedUsers.Append(*AvoidList);
		return TOnlineResult<FGetBlockedUsers>(MoveTemp(Result));
	}
	return TOnlineResult<FGetBlockedUsers>(Errors::InvalidState());
}


} // namespace UE::Online

#endif // WITH_GRDK
