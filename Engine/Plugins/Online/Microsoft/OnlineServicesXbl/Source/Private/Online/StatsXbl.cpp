// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK

#include "Online/StatsXbl.h"
#include "Online/AuthXbl.h"
#include "Online/OnlineUtils.h"
#include "Online/OnlineUtilsCommon.h"
#include "GDKRuntimeModule.h"
#include "Online/Windows/WindowsOnlineErrorDefinitions.h"

THIRD_PARTY_INCLUDES_START
#include <xsapi-c/user_statistics_c.h>
#include <xsapi-c/title_managed_statistics_c.h>
THIRD_PARTY_INCLUDES_END


namespace UE::Online 
{
	void LexFromString(EXblStatMode& OutMode, const TCHAR* InStr)
	{
		OutMode = (FCString::Stricmp(InStr, TEXT("Mode2017")) == 0) ? EXblStatMode::Mode2017 : EXblStatMode::Default;
	}

	bool ParseStatValue(FStatValue& TargetUserStat, const char* InStatisticValue, const char* InStatisticType)
	{
		const FString StatisticType = UTF8_TO_TCHAR(InStatisticType);
		const FString StatisticValue = UTF8_TO_TCHAR(InStatisticValue);

		if (FCString::Strcmp(*StatisticType, TEXT("Int32")) == 0)
		{
			TargetUserStat.Set(static_cast<int64>(FCString::Atoi(*StatisticValue)));
			return true;
		}
		if (FCString::Strcmp(*StatisticType, TEXT("UInt32")) == 0)
		{
			TargetUserStat.Set(static_cast<int64>(FCString::Strtoi(*StatisticValue, nullptr, 10))); // truncation should be fine if type is guaranteed
			return true;
		}
		if (FCString::Strcmp(*StatisticType, TEXT("Int64")) == 0 || FCString::Strcmp(*StatisticType, TEXT("Integer")) == 0) //Integers can also be read with this non-specific type
		{
			// Default integer type will be Int64
			TargetUserStat.Set(FCString::Atoi64(*StatisticValue));
			return true;
		}
		if (FCString::Strcmp(*StatisticType, TEXT("UInt64")) == 0)
		{
			TargetUserStat.Set(static_cast<int64>(FCString::Strtoui64(*StatisticValue, nullptr, 10)));
			return true;
		}
		if (FCString::Strcmp(*StatisticType, TEXT("Float")) == 0)
		{
			TargetUserStat.Set(FCString::Atof(*StatisticValue));
			return true;
		}
		if (FCString::Strcmp(*StatisticType, TEXT("Double")) == 0)
		{
			TargetUserStat.Set(FCString::Atod(*StatisticValue));
			return true;
		}
		if (FCString::Strcmp(*StatisticType, TEXT("Bool")) == 0)
		{
			TargetUserStat.Set(FCString::Strcmp(*StatisticValue, TEXT("False")) != 0);
			return true;
		}
		if (FCString::Strcmp(*StatisticType, TEXT("UnicodeString")) == 0)
		{
			TargetUserStat.Set(*StatisticValue);
			return true;
		}		
		return false;
	}

	void FStatsXbl::Initialize()
	{
		FStatsCommon::Initialize();
		TOnlineComponent::LoadConfig(Config);
		StatUpdateHandle = static_cast<FOnlineServicesXbl&>(Services).ContextManager->OnStatUpdate().Add(this, &FStatsXbl::OnlineStatUpdate);
	}

	void FStatsXbl::PreShutdown()
	{
		StatUpdateHandle.Unbind();
		FStatsCommon::PreShutdown();		
	}

	void FStatsXbl::OnlineStatUpdate(const FOnlineStatUpdate& Update)
	{
		FAccountId LocalAccountID = FOnlineAccountIdRegistryXbl::Get().Find(Update.XUID);
		if (!LocalAccountID.IsValid())
		{
			return;
		}
		TArray<FUserStats> UpdateUsersStats;


		FUserStats* LocalUserStats = CachedUsersStats.FindByPredicate([&LocalAccountID](FUserStats& UserStats)
			{
				return UserStats.AccountId == LocalAccountID;
			});
		if (!LocalUserStats)
		{
			return;
		}
		if (!LocalUserStats->Stats.Contains(Update.StatisticName))
		{
			return;
		}
		if (!ParseStatValue(LocalUserStats->Stats[Update.StatisticName], TCHAR_TO_ANSI(*Update.StatisticValue), TCHAR_TO_ANSI(*Update.StatisticType)))
		{
			return;
		}
		UpdateUsersStats.AddDefaulted(1);
		UpdateUsersStats[0].AccountId = LocalUserStats->AccountId;
		UpdateUsersStats[0].Stats.Emplace(Update.StatisticName, LocalUserStats->Stats[Update.StatisticName]);
		OnStatsUpdatedEvent.Broadcast({ LocalAccountID ,MoveTemp(UpdateUsersStats) });
	}

	TOnlineAsyncOpHandle<FTriggerEvent> FStatsXbl::TriggerEvent(FTriggerEvent::Params&& InParams)
	{
		TOnlineAsyncOpRef<FTriggerEvent> Op = GetOp<FTriggerEvent>(MoveTemp(InParams));

		if (!Op->IsReady())
		{
			Op->Then([this](TOnlineAsyncOp<FTriggerEvent>& Op)
				{
					const FTriggerEvent::Params& Params = Op.GetParams();
					if (!static_cast<FOnlineServicesXbl&>(Services).EventLauncher->TriggerEvent(Params.LocalAccountId, *Params.EventName, Params.EventParams))
					{
						Op.SetError(Errors::Unknown());
						return;
					}
					Op.SetResult({});
				}, FOnlineAsyncExecutionPolicy::RunOnThreadPool()).Enqueue(Services.GetParallelQueue());
		}
		return Op->GetHandle();
	}

	TOnlineAsyncOpHandle<FQueryStats> FStatsXbl::QueryStats(FQueryStats::Params&& InParams)
	{
		// Just call the batch OP with a size 1 batch.
		FBatchQueryStats::Params BatchParams;
		BatchParams.LocalAccountId = InParams.LocalAccountId;
		BatchParams.StatNames = InParams.StatNames;
		BatchParams.TargetAccountIds.Add(InParams.TargetAccountId);
		TOnlineAsyncOpRef<FBatchQueryStats> Op = BatchQueryStatsImpl(MoveTemp(BatchParams));

		return Op->GetWrappedHandle<FQueryStats>([](const TOnlineResult<FBatchQueryStats>& Result)
			{ 
				if (Result.IsOk())
				{
					const FUserStats& TargetUserStats = Result.GetOkValue().UsersStats[0];
					return TOnlineResult<FQueryStats>(FQueryStats::Result(TargetUserStats.Stats));
				}
				else
				{
					return TOnlineResult<FQueryStats>(Result.GetErrorValue());
				}
			});
	}

TOnlineAsyncOpRef<FBatchQueryStats> FStatsXbl::BatchQueryStatsImpl(FBatchQueryStats::Params&& InParams)
{
	TOnlineAsyncOpRef<FBatchQueryStats> Op = GetJoinableOp<FBatchQueryStats>(MoveTemp(InParams));
	if (Op->IsReady())
	{
		return Op;
	}
	Op->Then([this](TOnlineAsyncOp<FBatchQueryStats>& Op)
		{
			TSharedRef<TPromise<void>> Promise = MakeShared<TPromise<void>>();
			TFuture<void> Future = Promise->GetFuture();
			TSharedRef<FGDKAsyncBlock> AsyncBlock = MakeShared<FGDKAsyncBlock>(nullptr, [Promise](class FGDKAsyncBlock*)
				{
					//process results in next step
					Promise->EmplaceValue();
				});
			Op.Data.Set<TSharedRef<FGDKAsyncBlock>>(UE_XBL_ASYNC_BLOCK_KEY_NAME, AsyncBlock);

			const FBatchQueryStats::Params& Params = Op.GetParams();
			if (!Params.LocalAccountId.IsValid())
			{
				Op.SetError(Errors::InvalidParams());
				Promise->EmplaceValue();
				return Future;
			}

			if (!Services.Get<FAuthXbl>()->IsLoggedIn(Params.LocalAccountId))
			{
				Op.SetError(Errors::NotLoggedIn());
				Promise->EmplaceValue();
				return Future;
			}

			FGDKContextHandle GDKContext = static_cast<FOnlineServicesXbl&>(Services).ContextManager->GetGDKContext(FOnlineAccountIdRegistryXbl::Get().Find(Params.LocalAccountId));

			TArray<uint64> QueryUserIds;
			for (const FAccountId& Target : Params.TargetAccountIds)
			{
				uint64 XUID = FOnlineAccountIdRegistryXbl::Get().Find(Target);
				if (XUID == 0)
				{
					UE_LOGF(LogOnlineServices, Verbose, "[%s]: Failed to find XUID for user %ls", __FUNCTION__, *ToLogString(Target));
					continue;
				}
				QueryUserIds.Add(XUID);
			}

			TArray<TArray<ANSICHAR>> StatisticNamesStore;
			TArray<const ANSICHAR*> StatNamesCharPtr;

			for (const FString& StatName : Params.StatNames)
			{
				TArray<ANSICHAR>& AnsiCharArray = StatisticNamesStore.AddDefaulted_GetRef();
				AnsiCharArray.Append(TCHAR_TO_ANSI(*StatName), StatName.Len() + 1);
				StatNamesCharPtr.Add(AnsiCharArray.GetData());
			}

			const ANSICHAR* Scid = nullptr;
			XblGetScid(&Scid);

			HRESULT Result = XblUserStatisticsGetMultipleUserStatisticsAsync(GDKContext, QueryUserIds.GetData(),
				QueryUserIds.Num(), Scid, StatNamesCharPtr.GetData(), StatNamesCharPtr.Num(), *AsyncBlock);

			if (FAILED(Result))
			{
				FOnlineError Error = Errors::FromHRESULT(Result);
				Op.SetError(MoveTemp(Error));
				Promise->EmplaceValue();
				UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to query stats. Error %ls", __FUNCTION__, *Error.GetLogString());
				return Future;
			}

			Result = XblUserStatisticsTrackStatistics(GDKContext, QueryUserIds.GetData(), QueryUserIds.Num(), Scid,
				StatNamesCharPtr.GetData(), StatNamesCharPtr.Num());
			if (FAILED(Result))
			{
				FOnlineError Error = Errors::FromHRESULT(Result);
				UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to register for stat update events stats. Error %ls", __FUNCTION__, *Error.GetLogString())
			}
			return Future;
		})
		.Then([this](TOnlineAsyncOp<FBatchQueryStats>& Op)
		{
			const TSharedRef<FGDKAsyncBlock>& AsyncBlock = GetOpDataChecked<TSharedRef<FGDKAsyncBlock>>(Op, UE_XBL_ASYNC_BLOCK_KEY_NAME);

			uint64 ResultSizeInBytes = 0;
			HRESULT Result = XblUserStatisticsGetMultipleUserStatisticsResultSize(*AsyncBlock, &ResultSizeInBytes);
			if (FAILED(Result))
			{
				FOnlineError Error = Errors::FromHRESULT(Result);
				UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed get statistics query result size. Error %ls", __FUNCTION__, *Error.GetLogString());
				Op.SetError(MoveTemp(Error));
				return;
			}
			TArray<uint8> ResultBuffer;
			ResultBuffer.SetNumUninitialized(ResultSizeInBytes);

			XblUserStatisticsResult* StatisticsResultsArray{};
			uint64 ResultsCount = 0;
			uint64 BytesRead = 0;

			// If a value for this stat has never been set for this user, the query will return no entry for it
			Result = XblUserStatisticsGetMultipleUserStatisticsResult(*AsyncBlock, ResultSizeInBytes, ResultBuffer.GetData(), &StatisticsResultsArray, &ResultsCount, &BytesRead);
			if (FAILED(Result))
			{
				FOnlineError Error = Errors::FromHRESULT(Result);
				UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed get statistics query results. Error %ls", __FUNCTION__, *Error.GetLogString());
				Op.SetError(MoveTemp(Error));
				return;
			}

			const ANSICHAR* Scid = nullptr;
			XblGetScid(&Scid);
			TArray<FUserStats> NewStats;
			for (uint64 ResultsIndex = 0; ResultsIndex < ResultsCount; ++ResultsIndex) // Iterate over returned users
			{
				const XblUserStatisticsResult& StatisticsResultForUser = StatisticsResultsArray[ResultsIndex];
				FAccountId TargetAccountID = FOnlineAccountIdRegistryXbl::Get().Find(StatisticsResultForUser.xboxUserId);
				if (!TargetAccountID.IsValid())
				{
					continue;
				}

				FUserStats* TargetUserStats = CachedUsersStats.FindByPredicate([&TargetAccountID](FUserStats& UserStats)
					{
						return UserStats.AccountId == TargetAccountID;
					});
				if (!TargetUserStats)
				{
					int Index = CachedUsersStats.Emplace(TargetAccountID, TMap<FString, FStatValue>{});
					TargetUserStats = &CachedUsersStats[Index];
				}

				FUserStats& NewTargetUserStats = NewStats.Emplace_GetRef();
				NewTargetUserStats.AccountId = TargetAccountID;

				for (uint32 i = 0; i < StatisticsResultForUser.serviceConfigStatisticsCount; ++i) //Iterates over titles (we should just get our one)
				{
					const XblServiceConfigurationStatistic& ServiceConfigStat = StatisticsResultForUser.serviceConfigStatistics[i];

					if (strcmp(ServiceConfigStat.serviceConfigurationId, Scid) != 0)
					{
						continue;
					}

					for (uint32 j = 0; j < ServiceConfigStat.statisticsCount; ++j) //Iterate on stats
					{
						const XblStatistic& Stat = ServiceConfigStat.statistics[j];
						FString StatName = UTF8_TO_TCHAR(Stat.statisticName);
						FStatValue& NewStat = TargetUserStats->Stats.FindOrAdd(StatName);
						if (ParseStatValue(NewStat, Stat.value, Stat.statisticType))
						{
							NewTargetUserStats.Stats.Add(StatName, NewStat);
						}
					}
				}

			}
			FBatchQueryStats::Result OpResult;
			OpResult.UsersStats = MoveTemp(NewStats);
			Op.SetResult(MoveTemp(OpResult));
			return;
		});
	Op->Enqueue(Services.GetParallelQueue());
	return Op;
}

TOnlineAsyncOpHandle<FBatchQueryStats> FStatsXbl::BatchQueryStats(FBatchQueryStats::Params&& InParams)
{
	TOnlineAsyncOpRef<FBatchQueryStats> Op = BatchQueryStatsImpl(MoveTemp(InParams));
	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FUpdateStats> FStatsXbl::UpdateStats(FUpdateStats::Params&& Params)
{
	TOnlineAsyncOpRef<FUpdateStats> Op = GetOp<FUpdateStats>(MoveTemp(Params));

	if (!Services.Get<FAuthXbl>()->IsLoggedIn(Op->GetParams().LocalAccountId))
	{
		Op->SetError(Errors::InvalidUser());
		return Op->GetHandle();
	}

	for (const FUserStats& UpdateUserStats : Op->GetParams().UpdateUsersStats)
	{
		if (Op->GetParams().LocalAccountId != UpdateUserStats.AccountId)
		{
			// Client can only update the stats of the local account itself
			Op->SetError(Errors::InvalidParams());
			return Op->GetHandle();
		}
	}

	if (Config.StatMode == EXblStatMode::Mode2017)
	{
		UE_LOGF(LogOnlineServices, Error, "[%s]: Cannot update stats in event driven stats mode. Either reconfigure to Title managed stats or use TriggerEvent interface", __FUNCTION__);
		Op->SetError(Errors::InvalidState());
	}
	else
	{
		Op->Then([this](TOnlineAsyncOp<FUpdateStats>& InAsyncOp)
			{
				TSharedRef<TPromise<void>> Promise = MakeShared<TPromise<void>>();
				TFuture<void> Future = Promise->GetFuture();

				TSharedRef<TArray<TSharedRef<FGDKAsyncBlock>>> AsyncBlocks = MakeShared<TArray<TSharedRef<FGDKAsyncBlock>>>();

				InAsyncOp.Data.Set<TSharedRef<TArray<TSharedRef<FGDKAsyncBlock>>>>(UE_XBL_ASYNC_BLOCK_KEY_NAME, AsyncBlocks);
				const FUpdateStats::Params& Params = InAsyncOp.GetParams();
				for (const FUserStats& UserStats : Params.UpdateUsersStats)
				{
					TArray<XblTitleManagedStatistic> StatisticArray;
					FGDKContextHandle GDKContext = static_cast<FOnlineServicesXbl&>(Services).ContextManager->GetGDKContext(FOnlineAccountIdRegistryXbl::Get().Find(UserStats.AccountId));
					uint64 XUID = FOnlineAccountIdRegistryXbl::Get().Find(UserStats.AccountId);
					if (!GDKContext.IsValid())
					{
						UE_LOGF(LogOnlineServices, Warning, "[%s]: No context for user [%lld], are they local ?", __FUNCTION__, XUID);
						continue;
					}
					TArray<TArray<ANSICHAR>> CharBuffers;
					for (const TPair<FString, FStatValue>& StatPair : UserStats.Stats)
					{
						XblTitleManagedStatistic& CurrentStatistic = StatisticArray.AddZeroed_GetRef();

						switch (StatPair.Value.GetType())
						{
						case ESchemaAttributeType::Bool:
						{
							CurrentStatistic.statisticType = XblTitleManagedStatType::String;
							CurrentStatistic.numberValue = 0;
							TArray<ANSICHAR>& CharBuffer = CharBuffers.AddDefaulted_GetRef();
							const auto ConvertedStatName = TStringConversion<FTCHARToUTF8_Convert>(StatPair.Value.GetBoolean() ? TEXT("True") : TEXT("False"));
							CharBuffer.Append(ConvertedStatName.Get(), FCStringAnsi::Strlen(ConvertedStatName.Get()));
							CurrentStatistic.stringValue = CharBuffer.GetData();
							break;
						}
						case ESchemaAttributeType::String:
						{
							CurrentStatistic.statisticType = XblTitleManagedStatType::String;
							CurrentStatistic.numberValue = 0;
							TArray<ANSICHAR>& CharBuffer = CharBuffers.AddDefaulted_GetRef();
							const auto ConvertedStatName = TStringConversion<FTCHARToUTF8_Convert>(*StatPair.Value.GetString());
							CharBuffer.Append(ConvertedStatName.Get(), FCStringAnsi::Strlen(ConvertedStatName.Get()));
							CurrentStatistic.stringValue = CharBuffer.GetData();
							break;

						}
						case ESchemaAttributeType::Double:
						{
							CurrentStatistic.statisticType = XblTitleManagedStatType::Number;
							CurrentStatistic.numberValue = StatPair.Value.GetDouble();
							CurrentStatistic.stringValue = "";
							break;
						}
						case ESchemaAttributeType::Int64:
						{
							CurrentStatistic.statisticType = XblTitleManagedStatType::Number;
							CurrentStatistic.numberValue = StatPair.Value.GetInt64();
							CurrentStatistic.stringValue = "";
							break;
						}
						default: checkNoEntry(); // Intentional fallthrough						
						}
					}

					TSharedRef<FGDKAsyncBlock> AsyncBlock = MakeShared<FGDKAsyncBlock>(nullptr, [AsyncBlocks, Promise](class FGDKAsyncBlock*)
						{
							// progress once all tasks have completed
							bool bAllDone = true;
							for (TSharedRef<FGDKAsyncBlock>& Block : AsyncBlocks.Get())
							{
								if (Block->GetStatus() == E_PENDING)
								{
									bAllDone = false;
									break;
								}
							}
							Promise->EmplaceValue();
						});
					AsyncBlocks->Push(AsyncBlock);
					HRESULT Result = XblTitleManagedStatsUpdateStatsAsync(GDKContext, StatisticArray.GetData(), StatisticArray.Num(), *AsyncBlock);

					if (FAILED(Result))
					{
						FOnlineError Error = Errors::FromHRESULT(Result);
						InAsyncOp.SetError(MoveTemp(Error));
						Promise->EmplaceValue();
						UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to update stats. Error %ls", __FUNCTION__, *Error.GetLogString());
						return Future;
					}
	
				}
				return Future;

			}, FOnlineAsyncExecutionPolicy::RunOnThreadPool())
			.Then([this](TOnlineAsyncOp<FUpdateStats>& InAsyncOp)
			{
				const TSharedRef<TArray<TSharedRef<FGDKAsyncBlock>>>& AsyncBlocks = GetOpDataChecked< TSharedRef<TArray<TSharedRef<FGDKAsyncBlock>>>>(InAsyncOp, UE_XBL_ASYNC_BLOCK_KEY_NAME);
				int Successes = AsyncBlocks->Num();
				FOnlineError Error = Errors::Success();
				for (TSharedRef<FGDKAsyncBlock>& Block : AsyncBlocks.Get())
				{
					if (FAILED(Block->GetStatus()))
					{
						Error = Errors::FromHRESULT(Block->GetStatus());
						Successes--;
					}
				}
				if (Successes != AsyncBlocks->Num())
				{
					InAsyncOp.SetError(MoveTemp(Error));
					UE_LOGF(LogOnlineServices, Warning, "[%s]: Stat update failures [%d] of [%d], last error %ls", __FUNCTION__, Successes, AsyncBlocks->Num(), *Error.GetLogString());
				}
				else
				{
					InAsyncOp.SetResult({});
				}

			}, FOnlineAsyncExecutionPolicy::RunOnGameThread());
	}

	Op->Enqueue(GetSerialQueue());

	return Op->GetHandle();
}



/* UE::Online */ }

#endif // WITH_GRDK

