// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "Online/TitleFileXbl.h"

#include "Online/OnlineServicesXbl.h"
#include "Online/AuthXbl.h"
#include "Online/OnlineUtilsCommon.h"
#include "Online/Windows/WindowsOnlineErrorDefinitions.h"

#define UE_XBL_FILES_BUFFER_KEY_NAME  TEXT("FileBuffer")
#define UE_XBL_FILES_ID_KEY_NAME  TEXT("FileIDs")
#define UE_XBL_FILES_HANDLE_KEY_NAME  TEXT("FilesHandle")

namespace UE::Online {

FTitleFileXbl::FTitleFileXbl(FOnlineServicesXbl& InServices)
	: Super(InServices)
{
}

TOnlineResult<FTitleFileGetEnumeratedFiles> FTitleFileXbl::GetEnumeratedFiles(FTitleFileGetEnumeratedFiles::Params&& Params)
{
	if (!Services.Get<FAuthXbl>()->IsLoggedIn(Params.LocalAccountId))
	{
		return TOnlineResult<FTitleFileGetEnumeratedFiles>(Errors::InvalidUser());
	}

	TArray<FString> Filenames;	

	if(EnumeratedFiles.IsSet())
	{
		for (const XblTitleStorageBlobMetadata& FileData : EnumeratedFiles.GetValue())
		{
			Filenames.Emplace(FileData.blobPath);
		}
	}
	return TOnlineResult<FTitleFileGetEnumeratedFiles>({ MoveTemp(Filenames) });
}

TOnlineAsyncOpHandle<FTitleFileEnumerateFiles> FTitleFileXbl::EnumerateFiles(FTitleFileEnumerateFiles::Params&& InParams)
{
	TOnlineAsyncOpRef<FTitleFileEnumerateFiles> Op = GetJoinableOp<FTitleFileEnumerateFiles>(MoveTemp(InParams));
	if (!Op->IsReady())
	{
		Op->Then([this](TOnlineAsyncOp<FTitleFileEnumerateFiles>& Op)
			{
				const FTitleFileEnumerateFiles::Params& Params = Op.GetParams();
				TSharedRef<TPromise<void>> Promise = MakeShared<TPromise<void>>();
				TFuture<void> Future = Promise->GetFuture();

				if (!Services.Get<FAuthXbl>()->IsLoggedIn(Params.LocalAccountId))
				{
					Op.SetError(Errors::InvalidUser());
					return Future;
				}

				uint64 XUID = FOnlineAccountIdRegistryXbl::Get().Find(Params.LocalAccountId);
				if (XUID == 0)
				{
					Op.SetError(Errors::InvalidUser());
					return Future;
				}
				FGDKContextHandle GDKContext = static_cast<FOnlineServicesXbl&>(Services).ContextManager->GetGDKContext(XUID);
				if (!GDKContext.IsValid())
				{
					Op.SetError(Errors::InvalidUser());
					return Future;
				}

				TSharedRef<FGDKAsyncBlock> AsyncBlock = MakeShared<FGDKAsyncBlock>(nullptr, [Promise](class FGDKAsyncBlock*)
					{
						Promise->EmplaceValue();
					});
				Op.Data.Set<TSharedRef<FGDKAsyncBlock>>(UE_XBL_ASYNC_BLOCK_KEY_NAME, AsyncBlock);

				const ANSICHAR* Scid = nullptr;
				XblGetScid(&Scid);

				HRESULT Result = XblTitleStorageGetBlobMetadataAsync(GDKContext, Scid, XblTitleStorageType::GlobalStorage, 
					/*path*/"", XUID, /*skipItems*/0, /*maxItems 0 = no limt*/0, *AsyncBlock);
				if (FAILED(Result))
				{
					FOnlineError Error = Errors::FromHRESULT(Result);
					Op.SetError(MoveTemp(Error));
					UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to query title files. Error %ls", __FUNCTION__, *Error.GetLogString());
					return Future;
				}
					
				return Future;
			})
			.Then([this](TOnlineAsyncOp<FTitleFileEnumerateFiles>& Op)
			{
				const FTitleFileEnumerateFiles::Params& Params = Op.GetParams();
				const TSharedRef<FGDKAsyncBlock>& AsyncBlock = GetOpDataChecked<TSharedRef<FGDKAsyncBlock>>(Op, UE_XBL_ASYNC_BLOCK_KEY_NAME);
				XblTitleStorageBlobMetadataResultHandle DataResultHandle;
				HRESULT Result = XblTitleStorageGetBlobMetadataResult(*AsyncBlock, &DataResultHandle);
				if (FAILED(Result))
				{
					if (DataResultHandle)
					{
						XblTitleStorageBlobMetadataResultCloseHandle(DataResultHandle);
					}
					FOnlineError Error = Errors::FromHRESULT(Result);
					Op.SetError(MoveTemp(Error));
					UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to query title files. Error %ls", __FUNCTION__, *Error.GetLogString());
					return;
				}

				const XblTitleStorageBlobMetadata* MetaData;
				size_t FilesNum;
				Result = XblTitleStorageBlobMetadataResultGetItems(DataResultHandle, &MetaData, &FilesNum);
				if (FAILED(Result))
				{
					if (DataResultHandle)
					{
						XblTitleStorageBlobMetadataResultCloseHandle(DataResultHandle);
					}
					FOnlineError Error = Errors::FromHRESULT(Result);
					Op.SetError(MoveTemp(Error));
					UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to enumerate title files. Error %ls", __FUNCTION__, *Error.GetLogString());
					return;
				}	

				TSharedRef<TArray<XblTitleStorageBlobMetadata>> Files = MakeShared<TArray<XblTitleStorageBlobMetadata>>();
				Op.Data.Set<TSharedRef<TArray<XblTitleStorageBlobMetadata>>>(UE_XBL_FILES_ID_KEY_NAME, Files);

				for (int32 Index = 0; Index < FilesNum; ++Index)
				{
					(*Files).Add(MetaData[Index]);
				}

				Op.Data.Set<XblTitleStorageBlobMetadataResultHandle>(UE_XBL_FILES_HANDLE_KEY_NAME, DataResultHandle);
			})
			.Then([this](TOnlineAsyncOp<FTitleFileEnumerateFiles>& Op) mutable
			{
				XblTitleStorageBlobMetadataResultHandle DataResultHandle = GetOpDataChecked<XblTitleStorageBlobMetadataResultHandle>(Op, UE_XBL_FILES_HANDLE_KEY_NAME);
				TSharedRef<TPromise<TContinuationResult<void>>> Promise = MakeShared<TPromise<TContinuationResult<void>>>();
				TFuture<TContinuationResult<void>> Future = Promise->GetFuture();
				TSharedRef<FGDKAsyncBlock> AsyncBlock = MakeShared<FGDKAsyncBlock>(nullptr, [Promise, WeakOperation = TWeakPtr<TOnlineAsyncOp<FTitleFileEnumerateFiles>>(Op.AsShared())](class FGDKAsyncBlock* Block) mutable
					{
						TSharedPtr<TOnlineAsyncOp<FTitleFileEnumerateFiles>> Op = WeakOperation.Pin();
						if (Op)
						{
							XblTitleStorageBlobMetadataResultHandle DataResultHandle = GetOpDataChecked<XblTitleStorageBlobMetadataResultHandle>(*Op, UE_XBL_FILES_HANDLE_KEY_NAME);

							HRESULT Result = XblTitleStorageBlobMetadataResultGetNextResult(*Block, &DataResultHandle);
							if (Result != S_OK)
							{
								if (DataResultHandle)
								{
									XblTitleStorageBlobMetadataResultCloseHandle(DataResultHandle);
								}
								FOnlineError Error = Errors::FromHRESULT(Result);
								UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed get title file query result. Error %ls", __FUNCTION__, *Error.GetLogString());
								Op->SetError(MoveTemp(Error));
								return;
							}

							const XblTitleStorageBlobMetadata* MetaData;
							size_t FilesNum;
							Result = XblTitleStorageBlobMetadataResultGetItems(DataResultHandle, &MetaData, &FilesNum);
							if (FAILED(Result))
							{
								if (DataResultHandle)
								{
									XblTitleStorageBlobMetadataResultCloseHandle(DataResultHandle);
								}
								FOnlineError Error = Errors::FromHRESULT(Result);
								Op->SetError(MoveTemp(Error));
								UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to enumerate title files. Error %ls", __FUNCTION__, *Error.GetLogString());
								return;
							}

							const TSharedRef<TArray<XblTitleStorageBlobMetadata>>& Files = GetOpDataChecked<TSharedRef<TArray<XblTitleStorageBlobMetadata>>>(*Op, UE_XBL_FILES_ID_KEY_NAME);

							for (size_t Index = 0; Index < FilesNum; ++Index)
							{
								(*Files).Add(MetaData[Index]);
							}
							Op->Data.Set<XblTitleStorageBlobMetadataResultHandle>(UE_XBL_FILES_HANDLE_KEY_NAME, DataResultHandle);
						}
						Promise->EmplaceValue(TContinuationResult<void>::Repeat());
					});
				// Capture async block on operation.
				Op.Data.Set<TSharedRef<FGDKAsyncBlock>>(UE_XBL_ASYNC_BLOCK_KEY_NAME, AsyncBlock);

				if (DataResultHandle == nullptr)
				{
					Op.SetError(Errors::InvalidState());
					return Future;
				}
				bool bHasMore = false;
				HRESULT Result = XblTitleStorageBlobMetadataResultHasNext(DataResultHandle, &bHasMore);
				if (FAILED(Result))
				{
					if (DataResultHandle)
					{
						XblTitleStorageBlobMetadataResultCloseHandle(DataResultHandle);
					}
					FOnlineError Error = Errors::FromHRESULT(Result);
					UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed query title file pages. Error %ls", __FUNCTION__, *Error.GetLogString());
					Op.SetError(MoveTemp(Error));
					return Future;
				}
				if (!bHasMore)
				{
					XblTitleStorageBlobMetadataResultCloseHandle(DataResultHandle);
					Promise->EmplaceValue(TContinuationResult<void>::Complete());
					return Future;
				}

				Result = XblTitleStorageBlobMetadataResultGetNextAsync(DataResultHandle,0,*AsyncBlock);
				if (FAILED(Result))
				{
					if (DataResultHandle)
					{
						XblTitleStorageBlobMetadataResultCloseHandle(DataResultHandle);
					}
					FOnlineError Error = Errors::FromHRESULT(Result);
					UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed query title file page. Error %ls", __FUNCTION__, *Error.GetLogString());
					Op.SetError(MoveTemp(Error));
					return Future;
				}
				Op.Data.Set<XblTitleStorageBlobMetadataResultHandle>(UE_XBL_FILES_HANDLE_KEY_NAME, DataResultHandle);
				return Future;

			}, FOnlineAsyncExecutionPolicy::RunOnThreadPool())
			.Then([this](TOnlineAsyncOp<FTitleFileEnumerateFiles>& Op)
			{
				EnumeratedFiles = *GetOpDataChecked<TSharedRef<TArray<XblTitleStorageBlobMetadata>>>(Op, UE_XBL_FILES_ID_KEY_NAME);
				Op.SetResult({});
			})
			.Enqueue(GetSerialQueue());
	}
	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FTitleFileReadFile> FTitleFileXbl::ReadFile(FTitleFileReadFile::Params&& InParams)
{
	TOnlineAsyncOpRef<FTitleFileReadFile> Op = GetJoinableOp<FTitleFileReadFile>(MoveTemp(InParams));
	if (!Op->IsReady())
	{
		const FTitleFileReadFile::Params& Params = Op->GetParams();
		if (Params.Filename.IsEmpty())
		{
			Op->SetError(Errors::InvalidParams());
			return Op->GetHandle();
		}
		Op->Then([this](TOnlineAsyncOp<FTitleFileReadFile>& Op)
			{
				const FTitleFileReadFile::Params& Params = Op.GetParams();

				if (!Services.Get<FAuthXbl>()->IsLoggedIn(Params.LocalAccountId))
				{
					Op.SetError(Errors::InvalidUser());
				}
			})
			.Then([this](TOnlineAsyncOp<FTitleFileReadFile>& Op)
			{
				const FTitleFileReadFile::Params& Params = Op.GetParams();
				TSharedRef<TPromise<void>> Promise = MakeShared<TPromise<void>>();
				TFuture<void> Future = Promise->GetFuture();

				TSharedRef<FGDKAsyncBlock> AsyncBlock = MakeShared<FGDKAsyncBlock>(nullptr, [Promise](class FGDKAsyncBlock*)
					{
						Promise->EmplaceValue();
					});

				// Capture async block on operation.
				Op.Data.Set<TSharedRef<FGDKAsyncBlock>>(UE_XBL_ASYNC_BLOCK_KEY_NAME, AsyncBlock);

				uint64 XUID = FOnlineAccountIdRegistryXbl::Get().Find(Params.LocalAccountId);
				if (XUID == 0)
				{
					Op.SetError(Errors::InvalidUser());
					Promise->EmplaceValue();
					return Future;
				}
				FGDKContextHandle GDKContext = static_cast<FOnlineServicesXbl&>(Services).ContextManager->GetGDKContext(XUID);
				if (!GDKContext.IsValid())
				{
					Op.SetError(Errors::InvalidUser());
					Promise->EmplaceValue();
					return Future;
				}

				XblTitleStorageBlobMetadata* TargetMetaData = nullptr;
				if (EnumeratedFiles.IsSet())
				{
					for (XblTitleStorageBlobMetadata& FileData : EnumeratedFiles.GetValue())
					{
						if(Params.Filename == FileData.blobPath)
						{
							TargetMetaData = &FileData;
							break;
						}							
					}
				}
				else
				{
					// call enumerate first.
					Op.SetError(Errors::InvalidState());
					Promise->EmplaceValue();
					return Future;
				}

				if(TargetMetaData == nullptr)
				{
					Op.SetError(Errors::InvalidParams());
					Promise->EmplaceValue();
					return Future;						
				}

				TSharedRef<TArray<uint8>> FileBuffer = MakeShared<TArray<uint8>>();
				Op.Data.Set<TSharedRef<TArray<uint8>>>(UE_XBL_FILES_BUFFER_KEY_NAME, FileBuffer);					
				FileBuffer->AddUninitialized(TargetMetaData->length);

				HRESULT Result = XblTitleStorageDownloadBlobAsync(GDKContext, *TargetMetaData, FileBuffer->GetData(), TargetMetaData->length,
					XblTitleStorageETagMatchCondition::NotUsed, nullptr, 0, *AsyncBlock);

				if (FAILED(Result))
				{
					FOnlineError Error = Errors::FromHRESULT(Result);
					UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed download file. Error %ls", __FUNCTION__, *Error.GetLogString());
					Op.SetError(MoveTemp(Error));
					Promise->EmplaceValue();
					return Future;
				}				
				return Future;
			})
			.Then([this](TOnlineAsyncOp<FTitleFileReadFile>& Op)
			{
				const TSharedRef<FGDKAsyncBlock>& AsyncBlock = GetOpDataChecked<TSharedRef<FGDKAsyncBlock>>(Op, UE_XBL_ASYNC_BLOCK_KEY_NAME);

				if(FAILED(AsyncBlock->GetStatus()))
				{
					FOnlineError Error = Errors::FromHRESULT(AsyncBlock->GetStatus());
					UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed download file. Error %ls", __FUNCTION__, *Error.GetLogString());
					Op.SetError(MoveTemp(Error));
				}

				FTitleFileContentsRef FileContents = GetOpDataChecked<TSharedRef<TArray<uint8>>>(Op, UE_XBL_FILES_BUFFER_KEY_NAME);
				Op.SetResult({ MoveTemp(FileContents) });
					
			})
			.Enqueue(GetSerialQueue());
	}
	return Op->GetHandle();
}

/* UE::Online */ }
#endif // WITH_GRDK
