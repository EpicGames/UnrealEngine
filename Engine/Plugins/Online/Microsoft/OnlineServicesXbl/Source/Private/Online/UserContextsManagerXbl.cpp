// Copyright Epic Games, Inc. All Rights Reserved.
#if WITH_GRDK
#include "Online/UserContextsManagerXbl.h"
#include "Online/OnlineBase.h"

#include "Online/OnlineServicesXbl.h"
#include "Online/OnlineServicesLog.h"
#include "Online/Windows/WindowsOnlineErrorDefinitions.h"
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"
#include "Online/AchievementsXbl.h"
THIRD_PARTY_INCLUDES_START
#include <xsapi-c/presence_c.h>
#include <xsapi-c/achievements_c.h>
#include <xsapi-c/user_statistics_c.h>
THIRD_PARTY_INCLUDES_END


namespace UE::Online { 

void FUserContextsManagerXbl::CreateGDKContext(FGDKUserHandle GDKUser)
{
	uint64 XUID = 0;
	HRESULT Result = XUserGetId(GDKUser, &XUID);
	if (Result != S_OK)
	{
		UE_LOGF(LogOnlineServices, Error, "[%s]: Failed with error %ls", __FUNCTION__, *Errors::InvalidParams().GetLogString());
		return;
	}
	FScopeLock ScopeLock(&GDKContextsLock);

	FGDKContextInfo* GDKContextInfoPtr = CachedGDKContexts.Find(XUID);
	if (ensure(GDKContextInfoPtr == nullptr))
	{
		FGDKContextHandle GDKContext;
		TWeakPtr<FUserContextsManagerXbl, ESPMode::ThreadSafe> LambdaWeakThis = AsShared();

		Result = XblContextCreateHandle(GDKUser, GDKContext.GetInitReference());
		if (GDKContext)
		{
			FGDKContextInfo& ContextInfo = CachedGDKContexts.Emplace(XUID, GDKContext);

			// register for device presence updates. Things like online/offline status, playing on xbox/pc
			auto DevicePresenceChangedHandler = [](void* Context, uint64_t XUID, XblPresenceDeviceType deviceType, bool IsUserLoggedOnDevice)
				{
					UE_LOGF(LogOnlineServices,Verbose, "Received DevicePresenceChanged Event Player:%lld DeviceType:%d IsUserLoggedIn:%d", XUID, EnumToUnderlyingType(deviceType), IsUserLoggedOnDevice);

					FOnlineServicesXbl* Service = reinterpret_cast<FOnlineServicesXbl*>(Context);
					
					Service->ExecuteOnGameThread([Service, XUID, deviceType, IsUserLoggedOnDevice]()
						{
							FOnlineStatusUpdate Update;
							Update.bOnline = IsUserLoggedOnDevice;
							Update.XUID = XUID;
							Service->ContextManager->OnOnlineStatusUpdateEvent.Broadcast(Update);
						});
				};

			ContextInfo.DevicePresenceChangedContext = XblPresenceAddDevicePresenceChangedHandler(GDKContext, DevicePresenceChangedHandler, Service);

			//register for title Presence, things like rich presence
			auto TitlePresenceChangedHandler = [](void* Context, uint64 XUID, uint32 InTitleId, XblPresenceTitleState InTitleState)
				{
					UE_LOGF(LogOnlineServices, Verbose, "Received TitlePresenceChanged Event Player: %lld TitleId: %u TitleState:%d", XUID, InTitleId, EnumToUnderlyingType(InTitleState));

					FOnlineServicesXbl* Service = reinterpret_cast<FOnlineServicesXbl*>(Context);

					Service->ExecuteOnGameThread([Service, XUID, InTitleId, InTitleState]()
						{
							FTitleStatusUpdate Update;
							Update.TitleId = InTitleId;
							Update.XUID = XUID;
							Update.TitleState = static_cast<uint32>(InTitleState);
							Service->ContextManager->OnTitleStatusUpdateEvent.Broadcast(Update);
						});
				};

			ContextInfo.TitlePresenceChangedContext = XblPresenceAddTitlePresenceChangedHandler(GDKContext, TitlePresenceChangedHandler, Service);

			
			int32 Idx = UserContextWrappers.Add(MakeShared<UserContextWrapper, ESPMode::ThreadSafe>(AsWeak(), XUID));
			ContextInfo.StatisticChangedContext = XblUserStatisticsAddStatisticChangedHandler(
				GDKContext,
				[](XblStatisticChangeEventArgs statisticChangeEventArgs, void* Context)
				{
					const char* SCID;
					HRESULT Result = XblGetScid(&SCID);
					if(strcmp(SCID,statisticChangeEventArgs.serviceConfigurationId)!=0)
					{// Stat not from this title
						return;
					}
					UserContextWrapper* pThis = reinterpret_cast<UserContextWrapper*>(Context);
					TWeakPtr<UserContextWrapper, ESPMode::ThreadSafe> WeakThis = pThis->AsShared();
					TSharedPtr<UserContextWrapper, ESPMode::ThreadSafe> StrongThis = WeakThis.Pin();
					TSharedPtr<FUserContextsManagerXbl, ESPMode::ThreadSafe> StrongManager = StrongThis->Manager.Pin();
					if (StrongThis.IsValid() && StrongManager.IsValid())
					{
						FString StatisticName = statisticChangeEventArgs.latestStatistic.statisticName;
						FString StatisticType = statisticChangeEventArgs.latestStatistic.statisticType;
						FString StatisticValue = statisticChangeEventArgs.latestStatistic.value;

						StrongManager->Service->ExecuteOnGameThread([StrongThis,StrongManager, StatisticName, StatisticType, StatisticValue]()
							{	
								StrongManager->OnStatUpdateEvent.Broadcast({ StrongThis->XUID, StatisticName, StatisticType, StatisticValue });
							});
					}
				},UserContextWrappers[Idx].Get());
			//register for Achievement unlocks
			auto AchievementChangedHandler = [](const XblAchievementProgressChangeEventArgs* Event, void* Context)
				{					
					UserContextWrapper* pThis = reinterpret_cast<UserContextWrapper*>(Context);
					TWeakPtr<UserContextWrapper, ESPMode::ThreadSafe> WeakThis = pThis->AsShared();
					TSharedPtr<UserContextWrapper, ESPMode::ThreadSafe> StrongThis = WeakThis.Pin();
					TSharedPtr<FUserContextsManagerXbl, ESPMode::ThreadSafe> StrongManager = StrongThis->Manager.Pin();
					if (StrongThis.IsValid() && StrongManager.IsValid())
					{		
						FAchievementUpdate Updates;
						Updates.XUID = StrongThis->XUID;
						for (int i=0; i<Event->entryCount ;i++)
						{
							Updates.AchievementUpdates.Emplace( Event->updatedAchievementEntries[i].achievementId,
								Event->updatedAchievementEntries[i].progressState == XblAchievementProgressState::Achieved ?
								1.0f : 0.0f,
								FDateTime::FromUnixTimestamp(Event->updatedAchievementEntries[i].progression.timeUnlocked) );
						}
						StrongManager->Service->ExecuteOnGameThread([StrongThis, StrongManager, AchievementUpdates = MoveTemp(Updates)]() mutable
							{
								StrongManager->OnAchievementUpdateEvent.Broadcast( MoveTemp(AchievementUpdates));
							});
					}
				
				};

			ContextInfo.AchievementChangedContext = XblAchievementsAddAchievementProgressChangeHandler(GDKContext, AchievementChangedHandler, UserContextWrappers[Idx].Get());
		}
		else
		{
			FOnlineError Error = Errors::FromHRESULT(Result);
			UE_LOGF(LogOnlineServices, Error, "[%s]: Context creation failed %ls", __FUNCTION__, *Error.GetLogString());
		}
	}
	else
	{
		UE_LOGF(LogOnlineServices, Error, "[%s]: Failed to create context %ls", __FUNCTION__, *Errors::InvalidUser().GetLogString());
	}
}

void FUserContextsManagerXbl::DeleteGDKContext(FGDKUserHandle GDKUser)
{
	FScopeLock ScopeLock(&GDKContextsLock);

	uint64_t XUID;
	if (FAILED(XUserGetId(GDKUser, &XUID)))
	{
		UE_LOGF(LogOnlineServices, Error, "[%s]: Error %ls", __FUNCTION__, *Errors::InvalidUser().GetLogString());
		return;
	}

	FGDKContextInfo* GDKContextInfoPtr = CachedGDKContexts.Find(XUID);
	if (ensure(GDKContextInfoPtr))
	{
		UnsubscribeContextHandles(XUID, *GDKContextInfoPtr);
		GDKContextInfoPtr->Handle.Clear();
		CachedGDKContexts.Remove(XUID);
	}
	for (UserContextWrapperPtr UserContextWrapper : UserContextWrappers)
	{
		if (UserContextWrapper->XUID == XUID)
		{
			UserContextWrappers.Remove(UserContextWrapper);
			break;
		}
	}
}

void FUserContextsManagerXbl::UnsubscribeContextHandles(uint64 XUID, FGDKContextInfo& ContextInfo) const
{
	XblAchievementsRemoveAchievementProgressChangeHandler(ContextInfo.Handle, ContextInfo.AchievementChangedContext);
	XblUserStatisticsRemoveStatisticChangedHandler(ContextInfo.Handle, ContextInfo.StatisticChangedContext);
	XblPresenceRemoveTitlePresenceChangedHandler(ContextInfo.Handle, ContextInfo.TitlePresenceChangedContext);
	XblPresenceRemoveDevicePresenceChangedHandler(ContextInfo.Handle, ContextInfo.DevicePresenceChangedContext);
}
                                              
void FUserContextsManagerXbl::Shutdown()
{
	for (TPair<uint64, FGDKContextInfo>& GDKContextInfo : CachedGDKContexts)
	{
		UnsubscribeContextHandles(GDKContextInfo.Key, GDKContextInfo.Value);
	}
	CachedGDKContexts.Reset();
}

FGDKContextHandle FUserContextsManagerXbl::GetGDKContext(FPlatformUserId UserId)
{
	FGDKUserHandle UserHandle = IGDKRuntimeModule::Get().GetUserHandleByPlatformId(UserId);
	if (!UserHandle)
	{
		UE_LOGF(LogOnlineServices, Error, "[%s]: There is no GDKContext for the user id", __FUNCTION__);
		return FGDKContextHandle();
	}
	return GetGDKContext(UserHandle);
}

FGDKContextHandle FUserContextsManagerXbl::GetGDKContext(uint64 XUID)
{
	FScopeLock ScopeLock(&GDKContextsLock);

	FGDKContextInfo* GDKContextInfoPtr = CachedGDKContexts.Find(XUID);
	if (GDKContextInfoPtr)
	{
		check(GDKContextInfoPtr->Handle.IsValid());
		return GDKContextInfoPtr->Handle;
	}
	UE_LOGF(LogOnlineServices, Warning, "[%s]: There is no GDKContext for user this XUID - maybe user has already signed out", __FUNCTION__);

	return FGDKContextHandle();
}


FGDKContextHandle FUserContextsManagerXbl::GetGDKContext(FGDKUserHandle GDKUser)
{
	if (!GDKUser.IsValid())
	{
		UE_LOGF(LogOnlineServices, Error, "[%s]: Falied to get GDKContexthandle Error %ls", __FUNCTION__, *Errors::InvalidParams().GetLogString());
		return FGDKContextHandle();
	}

	uint64 UserId = 0;
	HRESULT Result = XUserGetId(GDKUser, &UserId);
	if (Result != S_OK)
	{
		UE_LOGF(LogOnlineServices, Error, "[%s]: There is no GDKContext for user XUID", __FUNCTION__ );
		return FGDKContextHandle();
	}

	return GetGDKContext(UserId);
}

TOnlineEvent<void(const FTitleStatusUpdate&)> FUserContextsManagerXbl::OnTitleStatusUpdate()
{
	return OnTitleStatusUpdateEvent;
}

TOnlineEvent<void(const FOnlineStatusUpdate&)> FUserContextsManagerXbl::OnOnlineStatusUpdate()
{
	return OnOnlineStatusUpdateEvent;
}

TOnlineEvent<void(const FOnlineStatUpdate&)> FUserContextsManagerXbl::OnStatUpdate()
{
	return OnStatUpdateEvent;
}

TOnlineEvent<void(const FAchievementUpdate&)> FUserContextsManagerXbl::OnAchievementUpdate()
{
	return OnAchievementUpdateEvent;
}

} // namespace UE::Online

#endif // WITH_GRDK
