// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK
#include "Online/UserFileXbl.h"

#include "Online/AuthXbl.h"
#include "Online/OnlineServicesXbl.h"
#include "Online/OnlineUtilsCommon.h"
#include "PlatformFeatures.h"
#include "SaveGameSystem.h"

#define UE_XBL_FILENAMES_KEY_NAME "Filenames"
#define UE_XBL_DATA_KEY_NAME "Data"
namespace UE::Online {


FUserFileXbl::FUserFileXbl(FOnlineServicesXbl& InServices)
	: Super(InServices)
{
}

TOnlineAsyncOpHandle<FUserFileEnumerateFiles> FUserFileXbl::EnumerateFiles(FUserFileEnumerateFiles::Params&& InParams)
{
	TOnlineAsyncOpRef<FUserFileEnumerateFiles> Op = GetJoinableOp<FUserFileEnumerateFiles>(MoveTemp(InParams));
	if (!Op->IsReady())
	{
		Op->Then([this](TOnlineAsyncOp<FUserFileEnumerateFiles>& Op)
		{
			const FUserFileEnumerateFiles::Params& Params = Op.GetParams();

			if (!Services.Get<FAuthXbl>()->IsLoggedIn(Params.LocalAccountId))
			{
				Op.SetError(Errors::InvalidUser());
			}
		})
		.Then([this](TOnlineAsyncOp<FUserFileEnumerateFiles>& Op)
		{
			const FUserFileEnumerateFiles::Params& Params = Op.GetParams();
			ISaveGameSystem* SaveGameSystem = IPlatformFeaturesModule::Get().GetSaveGameSystem();
			TSharedRef<TPromise<void>> Promise = MakeShared<TPromise<void>>();
			TFuture<void> Future = Promise->GetFuture();

			if(!SaveGameSystem)
			{
				Op.SetError(Errors::InvalidState());
				Promise->EmplaceValue();
				return Future;
			}
			auto  Callback = [Promise, WeakOperation = TWeakPtr<TOnlineAsyncOp<FUserFileEnumerateFiles>>(Op.AsShared())](FPlatformUserId User,bool bSuccess, const TArray<FString>& Results)
				{
					TSharedPtr<TOnlineAsyncOp<FUserFileEnumerateFiles>> Op = WeakOperation.Pin();
					if (Op)
					{
						if (!bSuccess)
						{
							Op->SetError(Errors::Unknown());
						}
						else
						{
							Op->Data.Set<TArray<FString>>(UE_XBL_FILENAMES_KEY_NAME, Results);
						}
					}
					Promise->EmplaceValue();
				};			

			SaveGameSystem->GetSaveGameNamesAsync(
				Services.Get<FAuthXbl>()->GetAccountInfoRegistry().Find(Params.LocalAccountId)->PlatformUserId
				, Callback);	
			return Future;
		})
		.Then([this](TOnlineAsyncOp<FUserFileEnumerateFiles>& Op)
		{
			const FUserFileEnumerateFiles::Params& Params = Op.GetParams();
			UserToFiles.Emplace(Params.LocalAccountId, GetOpDataChecked<TArray<FString>>(Op, UE_XBL_FILENAMES_KEY_NAME));
			Op.SetResult({});
		})
		.Enqueue(GetSerialQueue());
	}
	return Op->GetHandle();
}

TOnlineResult<FUserFileGetEnumeratedFiles> FUserFileXbl::GetEnumeratedFiles(FUserFileGetEnumeratedFiles::Params&& Params)
{
	if (!Services.Get<FAuthXbl>()->IsLoggedIn(Params.LocalAccountId))
	{
		return TOnlineResult<FUserFileGetEnumeratedFiles>(Errors::InvalidUser());
	}

	const TArray<FString>* Found = UserToFiles.Find(Params.LocalAccountId);
	if (!Found)
	{
		// Call EnumerateFiles first
		return TOnlineResult<FUserFileGetEnumeratedFiles>(Errors::InvalidState());
	}

	return TOnlineResult<FUserFileGetEnumeratedFiles>({*Found});
}

TOnlineAsyncOpHandle<FUserFileReadFile> FUserFileXbl::ReadFile(FUserFileReadFile::Params&& InParams)
{
	TOnlineAsyncOpRef<FUserFileReadFile> Op = GetJoinableOp<FUserFileReadFile>(MoveTemp(InParams));
	if (!Op->IsReady())
	{
		const FUserFileReadFile::Params& Params = Op->GetParams();
		if (Params.Filename.IsEmpty())
		{
			Op->SetError(Errors::InvalidParams());
			return Op->GetHandle();
		}
		Op->Then([this](TOnlineAsyncOp<FUserFileReadFile>& Op)
			{
				const FUserFileReadFile::Params& Params = Op.GetParams();

				if (!Services.Get<FAuthXbl>()->IsLoggedIn(Params.LocalAccountId))
				{
					Op.SetError(Errors::InvalidUser());
				}
			})
			.Then([this](TOnlineAsyncOp<FUserFileReadFile>& Op)
				{
					const FUserFileReadFile::Params& Params = Op.GetParams();
					ISaveGameSystem* SaveGameSystem = IPlatformFeaturesModule::Get().GetSaveGameSystem();
					TSharedRef<TPromise<void>> Promise = MakeShared<TPromise<void>>();
					TFuture<void> Future = Promise->GetFuture();

					if (!SaveGameSystem)
					{
						Op.SetError(Errors::InvalidState());
						Promise->EmplaceValue();
						return Future;
					}
					auto  Callback = [Promise, WeakOperation = TWeakPtr<TOnlineAsyncOp<FUserFileReadFile>>(Op.AsShared())](const FString& Name, FPlatformUserId User, bool bSuccess, const TArray<uint8>& Data)
						{
							TSharedPtr<TOnlineAsyncOp<FUserFileReadFile>> Op = WeakOperation.Pin();
							if (Op)
							{
								if (!bSuccess)
								{
									Op->SetError(Errors::Unknown());
								}
								else
								{
									Op->Data.Set<TSharedPtr<TArray<uint8>>>(UE_XBL_DATA_KEY_NAME, MakeShareable(new TArray<uint8>(Data)));
								}
							}
							Promise->EmplaceValue();
						};

					SaveGameSystem->LoadGameAsync(
						false
						, *Params.Filename
						, Services.Get<FAuthXbl>()->GetAccountInfoRegistry().Find(Params.LocalAccountId)->PlatformUserId
						, Callback);
					return Future;
				})
			.Then([this](TOnlineAsyncOp<FUserFileReadFile>& Op)
				{
					const FUserFileContentsRef* FileContents = Op.Data.Get<FUserFileContentsRef>(UE_XBL_DATA_KEY_NAME);
					if (ensure(FileContents))
					{
						Op.SetResult({ *FileContents });
					}
					else
					{
						Op.SetError(Errors::Unknown());
					}
				})
			.Enqueue(GetSerialQueue());
	}
	return Op->GetHandle();
}

TOnlineAsyncOpHandle<FUserFileWriteFile> FUserFileXbl::WriteFile(FUserFileWriteFile::Params&& InParams)
{
	TOnlineAsyncOpRef<FUserFileWriteFile> Op = GetOp<FUserFileWriteFile>(MoveTemp(InParams));
	const FUserFileWriteFile::Params& Params = Op->GetParams();

	if (Params.Filename.IsEmpty())
	{
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}

	Op->Then([this](TOnlineAsyncOp<FUserFileWriteFile>& Op)
		{
			const FUserFileWriteFile::Params& Params = Op.GetParams();

			if (!Services.Get<FAuthXbl>()->IsLoggedIn(Params.LocalAccountId))
			{
				Op.SetError(Errors::InvalidUser());
			}
		})
		.Then([this](TOnlineAsyncOp<FUserFileWriteFile>& Op)
			{
				const FUserFileWriteFile::Params& Params = Op.GetParams();
				ISaveGameSystem* SaveGameSystem = IPlatformFeaturesModule::Get().GetSaveGameSystem();
				TSharedRef<TPromise<void>> Promise = MakeShared<TPromise<void>>();
				TFuture<void> Future = Promise->GetFuture();

				if (!SaveGameSystem)
				{
					Op.SetError(Errors::InvalidState());
					Promise->EmplaceValue();
					return Future;
				}
				auto  Callback = [Promise, WeakOperation = TWeakPtr<TOnlineAsyncOp<FUserFileWriteFile>>(Op.AsShared())](const FString& Name, FPlatformUserId User, bool bSuccess)
					{
						TSharedPtr<TOnlineAsyncOp<FUserFileWriteFile>> Op = WeakOperation.Pin();
						if (Op)
						{
							if (!bSuccess)
							{
								Op->SetError(Errors::Unknown());
							}
						}
						Promise->EmplaceValue();
					};			

				SaveGameSystem->SaveGameAsync(
					false
					, *Params.Filename
					, Services.Get<FAuthXbl>()->GetAccountInfoRegistry().Find(Params.LocalAccountId)->PlatformUserId
					, MakeShareable(new TArray<uint8>(Params.FileContents))
					, Callback);
				return Future;
			})
		.Then([this](TOnlineAsyncOp<FUserFileWriteFile>& Op)
			{
				const FUserFileWriteFile::Params& Params = Op.GetParams();
				Op.SetResult({});
			})
		.Enqueue(GetSerialQueue());
	return Op->GetHandle();
}


TOnlineAsyncOpHandle<FUserFileDeleteFile> FUserFileXbl::DeleteFile(FUserFileDeleteFile::Params&& InParams)
{
	TOnlineAsyncOpRef<FUserFileDeleteFile> Op = GetOp<FUserFileDeleteFile>(MoveTemp(InParams));
	const FUserFileDeleteFile::Params& Params = Op->GetParams();

	if (Params.Filename.IsEmpty())
	{
		Op->SetError(Errors::InvalidParams());
		return Op->GetHandle();
	}

	Op->Then([this](TOnlineAsyncOp<FUserFileDeleteFile>& Op)
		{
			const FUserFileDeleteFile::Params& Params = Op.GetParams();

			if (!Services.Get<FAuthXbl>()->IsLoggedIn(Params.LocalAccountId))
			{
				Op.SetError(Errors::InvalidUser());
			}
		})
		.Then([this](TOnlineAsyncOp<FUserFileDeleteFile>& Op)
			{
				const FUserFileDeleteFile::Params& Params = Op.GetParams();
				ISaveGameSystem* SaveGameSystem = IPlatformFeaturesModule::Get().GetSaveGameSystem();
				TSharedRef<TPromise<void>> Promise = MakeShared<TPromise<void>>();
				TFuture<void> Future = Promise->GetFuture();

				if (!SaveGameSystem)
				{
					Op.SetError(Errors::InvalidState());
					Promise->EmplaceValue();
					return Future;
				}
				auto Callback = [Promise, WeakOperation = TWeakPtr<TOnlineAsyncOp<FUserFileDeleteFile>>(Op.AsShared())](const FString& Name, FPlatformUserId User, bool bSuccess)
					{
						TSharedPtr<TOnlineAsyncOp<FUserFileDeleteFile>> Op = WeakOperation.Pin();
						if (Op)
						{
							if (!bSuccess)
							{
								Op->SetError(Errors::Unknown());
							}
						}
						Promise->EmplaceValue();
					};

				SaveGameSystem->DeleteGameAsync(
					false
					, *Params.Filename
					, Services.Get<FAuthXbl>()->GetAccountInfoRegistry().Find(Params.LocalAccountId)->PlatformUserId
					, Callback);
				return Future;
			})
		.Then([this](TOnlineAsyncOp<FUserFileDeleteFile>& Op)
			{
				Op.SetResult({});
			})
		.Enqueue(GetSerialQueue());
	return Op->GetHandle();
}

/* UE::Online */ }
#endif // WITH_GRDK
