// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_GRDK

#include "Online/UserInfoXbl.h"
#include "Online/AuthXbl.h"
#include "Online/OnlineUtils.h"
#include "Online/OnlineUtilsCommon.h"
#include "GDKRuntimeModule.h"
#include "Online/Windows/WindowsOnlineErrorDefinitions.h"
THIRD_PARTY_INCLUDES_START
#include <xsapi-c/profile_c.h>
#include <XGameUI.h>
THIRD_PARTY_INCLUDES_END

namespace UE::Online {

TOnlineAsyncOpHandle<FQueryUserInfo> FUserInfoXbl::QueryUserInfo(FQueryUserInfo::Params&& InParams)
{
	TOnlineAsyncOpRef<FQueryUserInfo> Op = GetJoinableOp<FQueryUserInfo>(MoveTemp(InParams));
	if (!Op->IsReady())
	{
		const FQueryUserInfo::Params& Params = Op->GetParams();
		if (Params.AccountIds.IsEmpty())
		{
			Op->SetError(Errors::InvalidParams());
			return Op->GetHandle();
		}

		Op->Then([this](TOnlineAsyncOp<FQueryUserInfo>& Op)
			{
				TSharedRef<TPromise<void>> Promise = MakeShared<TPromise<void>>();
				TFuture<void> Future = Promise->GetFuture();

				TSharedRef<FGDKAsyncBlock> AsyncBlock = MakeShared<FGDKAsyncBlock>(nullptr, [Promise](class FGDKAsyncBlock*)
					{
						Promise->EmplaceValue();
					});
				// Capture async block on operation.
				Op.Data.Set<TSharedRef<FGDKAsyncBlock>>(UE_XBL_ASYNC_BLOCK_KEY_NAME, AsyncBlock);

				const FQueryUserInfo::Params& Params = Op.GetParams();

				TArray<uint64> XUIDs;
				for (const FAccountId& TargetAccountId : Params.AccountIds)
				{
					if (uint64 XUID = FOnlineAccountIdRegistryXbl::Get().Find(TargetAccountId))						
					{
						XUIDs.Add(XUID);
					}
					else
					{
						UE_LOGF(LogOnlineServices, Verbose, "[%s]: No XUIDs for local users  %ls", __FUNCTION__, *ToString(TargetAccountId));
					}
				}

				if (XUIDs.IsEmpty())
				{
					Op.SetError(Errors::NotLoggedIn());
					Promise->EmplaceValue();
					UE_LOGF(LogOnlineServices, Warning, "[%s]: No XUIDs for local users.", __FUNCTION__);
				}

				FGDKContextHandle GDKContext = static_cast<FOnlineServicesXbl&>(Services).ContextManager->GetGDKContext(FOnlineAccountIdRegistryXbl::Get().Find(Params.LocalAccountId));
				HRESULT Result = XblProfileGetUserProfilesAsync(GDKContext, XUIDs.GetData(), XUIDs.Num(), *AsyncBlock);
				if (FAILED(Result))
				{
					FOnlineError Error = Errors::FromHRESULT(Result);
					Op.SetError(MoveTemp(Error));
					Promise->EmplaceValue();
					UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to request user profiles. Error %ls", __FUNCTION__, *Error.GetLogString());
				}

				return Future;
			}).Then([this](TOnlineAsyncOp<FQueryUserInfo>& Op)
			{
					const TSharedRef<FGDKAsyncBlock>& AsyncBlock = GetOpDataChecked<TSharedRef<FGDKAsyncBlock>>(Op, UE_XBL_ASYNC_BLOCK_KEY_NAME);
					size_t NumProfiles = 0;
					HRESULT Result = XblProfileGetUserProfilesResultCount(*AsyncBlock, &NumProfiles);
					if (FAILED(Result))
					{
						FOnlineError Error = Errors::FromHRESULT(Result);
						Op.SetError(MoveTemp(Error));
						UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to get user profiles count. Error %ls", __FUNCTION__, *Error.GetLogString());
						return;
					}
					TArray<XblUserProfile> UserProfiles;
					UserProfiles.Reserve(NumProfiles);
					Result = XblProfileGetUserProfilesResult(*AsyncBlock, NumProfiles, UserProfiles.GetData());
					if (FAILED(Result))
					{
						FOnlineError Error = Errors::FromHRESULT(Result);
						Op.SetError(MoveTemp(Error));
						UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to get user profiles. Error %ls", __FUNCTION__, *Error.GetLogString());
						return;
					}
					UserProfiles.SetNum(NumProfiles);

					for (XblUserProfile& Profile : UserProfiles)
					{
						TSharedRef<XblUserProfile> NewUser = MakeShared<XblUserProfile>(MoveTemp(Profile));
						UserProfileCache.Add(Profile.xboxUserId, NewUser);
					}
					Op.SetResult(FQueryUserInfo::Result{ });

			});

		Op->Enqueue(Services.GetParallelQueue());
	}
	return Op->GetHandle();
}

TOnlineResult<FGetUserInfo> FUserInfoXbl::GetUserInfo(FGetUserInfo::Params&& Params)
{
	if (!Services.Get<FAuthXbl>()->IsLoggedIn(Params.LocalAccountId))
	{
		return TOnlineResult<FGetUserInfo>(Errors::NotLoggedIn());
	}

	if (uint64 XUID = FOnlineAccountIdRegistryXbl::Get().Find(Params.AccountId))
	{
		if (TSharedPtr<XblUserProfile>* Profile = UserProfileCache.Find(XUID))
		{
			TSharedPtr<FAccountInfoXbl> AccountInfo = static_cast<const FAccountInfoRegistryXbl&>(Services.Get<FAuthXbl>()->GetAccountInfoRegistry()).Find(Params.AccountId);
			if (AccountInfo.IsValid())
			{
				TSharedRef<FUserInfo> UserInfo = MakeShared<FUserInfo>();
				UserInfo->AccountId = Params.AccountId;
				UserInfo->DisplayName = IGDKRuntimeModule::Get().GetGamertag(AccountInfo->UserHandle);
				return TOnlineResult<FGetUserInfo>({ UserInfo });
			}
		}
		return TOnlineResult<FGetUserInfo>(Errors::NotFound());
	}
	else
	{
		return TOnlineResult<FGetUserInfo>(Errors::InvalidUser());
	}
}

TOnlineAsyncOpHandle<FQueryUserAvatar> FUserInfoXbl::QueryUserAvatar(FQueryUserAvatar::Params&& InParams)
{
	TOnlineAsyncOpRef<FQueryUserAvatar> Op = GetJoinableOp<FQueryUserAvatar>(MoveTemp(InParams));

	if (!Op->IsReady())
	{
		const FQueryUserAvatar::Params& Params = Op->GetParams();
		if (Params.AccountIds.IsEmpty())
		{
			Op->SetError(Errors::InvalidParams());
			return Op->GetHandle();
		}

		Op->Then([this](TOnlineAsyncOp<FQueryUserAvatar>& Op)
			{
				const FQueryUserAvatar::Params& Params = Op.GetParams();
				FQueryUserInfo::Params InfoParams;
				InfoParams.LocalAccountId = Params.LocalAccountId;
				InfoParams.AccountIds = Params.AccountIds;
				UE::Online::TOnlineAsyncOpHandle<FQueryUserInfo> Infohandle = QueryUserInfo(MoveTemp(InfoParams));

				Infohandle.OnComplete(this,[&Op](const TOnlineResult<FQueryUserInfo>& InfoResult) mutable
					{
						if(InfoResult.IsError())
						{
							Op.SetError(FOnlineError(InfoResult.GetErrorValue()));
						}
						else
						{
							Op.SetResult(FQueryUserAvatar::Result{});
						}
					});
				});
		Op->Enqueue(Services.GetParallelQueue());

	}
	return Op->GetHandle();
}

TOnlineResult<FGetUserAvatar> FUserInfoXbl::GetUserAvatar(FGetUserAvatar::Params&& Params)
{
	if (!Services.Get<FAuthXbl>()->IsLoggedIn(Params.LocalAccountId))
	{
		return TOnlineResult<FGetUserAvatar>(Errors::NotLoggedIn());
	}

	if (uint64 XUID = FOnlineAccountIdRegistryXbl::Get().Find(Params.AccountId))
	{
		if(TSharedPtr<XblUserProfile>* Profile = UserProfileCache.Find(XUID))
		{
			return TOnlineResult<FGetUserAvatar>({ FString(UTF8_TO_TCHAR(Profile->Get()->gameDisplayPictureResizeUri))});
		}
		return TOnlineResult<FGetUserAvatar>(Errors::NotFound());		
	}
	else
	{
		return TOnlineResult<FGetUserAvatar>(Errors::InvalidUser());
	}
}

TOnlineAsyncOpHandle<FShowUserProfile> FUserInfoXbl::ShowUserProfile(FShowUserProfile::Params&& Params)
{
	TOnlineAsyncOpRef<FShowUserProfile> Operation = GetOp<FShowUserProfile>(MoveTemp(Params));

	TUniquePtr<FGDKAsyncBlock> AsyncBlock = MakeUnique<FGDKAsyncBlock>(nullptr, [](FGDKAsyncBlock* AsyncBlock)
		{
			delete AsyncBlock;
			//we don't care about the result.
		});

	uint64 TargetXUID = FOnlineAccountIdRegistryXbl::Get().Find(Params.AccountId);
	uint64 LocalXUID = FOnlineAccountIdRegistryXbl::Get().Find(Params.LocalAccountId);

	FGDKUserHandle UserHandle = IGDKRuntimeModule::Get().GetUserHandleByXUserId(LocalXUID);
	if (!UserHandle.IsValid())
	{
		Operation->SetError(Errors::InvalidUser());
	}

	if(TargetXUID==0 || LocalXUID==0)
	{
		Operation->SetError(Errors::InvalidParams());
	}	

	HRESULT Result = XGameUiShowPlayerProfileCardAsync(AsyncBlock->GetInnerBlockForGDKAPI(), UserHandle, TargetXUID);
	if (Result != S_OK)
	{
		AsyncBlock.Reset();
		FOnlineError Error = Errors::FromHRESULT(Result);
		UE_LOGF(LogOnlineServices, Warning, "[%s]: Failed to show UI. Error %ls", __FUNCTION__, *Error.GetLogString());
		Operation->SetError(MoveTemp(Error));
	}
	else
	{
		(void)AsyncBlock.Release();
		Operation->SetResult(FShowUserProfile::Result{});
	}

	return Operation->GetHandle();
}

/* UE::Online */ }

#endif // WITH_GRDK

