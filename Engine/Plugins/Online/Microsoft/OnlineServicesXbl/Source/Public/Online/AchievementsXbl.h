// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_GRDK

#include "Online/AchievementsCommon.h"
#include "Online/OnlineComponent.h"
#include "Serialization/JsonSerializerMacros.h"
#include "Online/UserContextsManagerXbl.h"


class FAchievementsConfig : public FJsonSerializable
{
public:
	FString								AchievementEventName;
	FJsonSerializableKeyValueMapInt		AchievementMap;

	BEGIN_JSON_SERIALIZER
		JSON_SERIALIZE("AchievementEventName", AchievementEventName);
		JSON_SERIALIZE_MAP("AchievementMap", AchievementMap);
	END_JSON_SERIALIZER
};

namespace UE::Online {

	enum class EXblAchievementMode : uint16
	{
		/** 2013 achievements are driven via events */
		Mode2013 = 2013,
		/** 2017 achievements are title-managed */
		Mode2017 = 2017,
		/** Default mode to use if not specified */
		Default = Mode2013
	};

	struct FAchievementsXblConfig
	{
		EXblAchievementMode AchivementMode = EXblAchievementMode::Default;
	};

	namespace Meta 
	{
		BEGIN_ONLINE_STRUCT_META(FAchievementsXblConfig)
			ONLINE_STRUCT_FIELD(FAchievementsXblConfig, AchivementMode)
			END_ONLINE_STRUCT_META()
			/* Meta */
	}

class ONLINESERVICESXBL_API FAchievementsXbl : public FAchievementsCommon
{
public:
	using Super = FAchievementsCommon;

	using FAchievementsCommon::FAchievementsCommon;

	virtual void Initialize() override;
	virtual void PreShutdown() override;

	// IAchievements
	virtual TOnlineAsyncOpHandle<FQueryAchievementDefinitions> QueryAchievementDefinitions(FQueryAchievementDefinitions::Params&& Params) override;
	virtual TOnlineResult<FGetAchievementIds> GetAchievementIds(FGetAchievementIds::Params&& Params) override;
	virtual TOnlineResult<FGetAchievementDefinition> GetAchievementDefinition(FGetAchievementDefinition::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FQueryAchievementStates> QueryAchievementStates(FQueryAchievementStates::Params&& Params) override;
	virtual TOnlineResult<FGetAchievementState> GetAchievementState(FGetAchievementState::Params&& Params) const override;
	virtual TOnlineAsyncOpHandle<FUnlockAchievements> UnlockAchievements(FUnlockAchievements::Params&& Params) override;
	// End IAchievements

	void AchievementUnlockNotificationHandler(const FAchievementUpdate& Update);

protected:

	const FString& GetAchievementNameFromId(int32 AchievementId);
	int32 GetAchievementIdFromName(const FString& AchievementName);
	const FString& GetAchievementEventName();
	bool LoadAndInitFromJsonConfig(const TCHAR* JsonConfigName);

	using FAchievementDefinitionMap = TMap<FString, FAchievementDefinition>;
	TOptional<FAchievementDefinitionMap> AchievementDefinitions;
	FAchievementsConfig	AchievementsConfig;
	FAchievementsXblConfig Config;
	FOnlineEventDelegateHandle AchievementUpdateHandle;

};

/* UE::Online */ }

#endif // WITH_GRDK
