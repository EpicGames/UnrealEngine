// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_GRDK

#include "Online/OnlineServicesXbl.h"
#include "CoreMinimal.h"
#include "HAL/CriticalSection.h"
#include "Online/AuthCommon.h"
#include "GDKHandle.h"
#include "GDKTaskQueueHelpers.h"

namespace UE::Online {

class FOnlineServicesXbl;


class FOnlineAccountIdXbl
{
public:
	FAccountId AccountId;
	uint64 XUID;
};

class FOnlineAccountIdRegistryXbl : public IOnlineAccountIdRegistry
{
public:
	static FOnlineAccountIdRegistryXbl& Get();

	FAccountId Find(const uint64 XUID) const;
	FAccountId FindOrAddAccountId(const uint64 XUID);

	uint64 Find(const FAccountId& AccountId) const;

	// Begin IOnlineAccountIdRegistry
	virtual FString ToString(const FAccountId& AccountId) const override;
	virtual FString ToLogString(const FAccountId& AccountId) const override;
	virtual TArray<uint8> ToReplicationData(const FAccountId& AccountId) const override;
	virtual FAccountId FromReplicationData(const TArray<uint8>& ReplicationString) override;
	virtual FAccountId FromStringData(const FString& StringData) override;
	// End IOnlineAccountIdRegistry

	virtual ~FOnlineAccountIdRegistryXbl() = default;

private:
	// Retrieve an entry with no lock. For internal use only.
	const FOnlineAccountIdXbl* FindNoLock(const FAccountId& AccountId) const;
	const FOnlineAccountIdXbl* FindNoLock(const uint64 XUID) const;

	mutable FRWLock Lock;
	TArray<FOnlineAccountIdXbl> Ids;
	TMap<uint64, FOnlineAccountIdXbl*> XUIDIndex;

};

struct FAccountInfoXbl final : public FAccountInfo
{
	FGDKUserHandle UserHandle;
};

class ONLINESERVICESXBL_API FAccountInfoRegistryXbl final : public FAccountInfoRegistry
{
public:
	using Super = FAccountInfoRegistry;

	virtual ~FAccountInfoRegistryXbl() = default;

	TSharedPtr<FAccountInfoXbl> FindOrCreate(FPlatformUserId PlatformUserId, FOnlineServicesXbl& Service);
	TSharedPtr<FAccountInfoXbl> Find(FPlatformUserId PlatformUserId) const;
	TSharedPtr<FAccountInfoXbl> Find(FAccountId AccountIdHandle) const;

	void Register(const TSharedRef<FAccountInfoXbl>&UserAuthData);
	void Unregister(FAccountId AccountId);
};

class ONLINESERVICESXBL_API FAuthXbl : public FAuthCommon
{
public:
	using Super = FAuthCommon;

	FAuthXbl(FOnlineServicesXbl& InOwningSubsystem);
	virtual void Initialize() override;
	virtual void PreShutdown() override;

	// Begin IAuth
	virtual TOnlineAsyncOpHandle<FAuthLogin> Login(FAuthLogin::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FAuthQueryExternalAuthToken> QueryExternalAuthToken(FAuthQueryExternalAuthToken::Params&& Params) override;
	// End IAuth


	virtual const FAccountInfoRegistry& GetAccountInfoRegistry() const override;
protected:

	void InitializeUsers();
	void UninitializeUsers();
	void OnInputDeviceConnectionChange(EInputDeviceConnectionState NewConnectionState, FPlatformUserId PlatformUserId, FInputDeviceId InputDeviceId);

	FAccountInfoRegistryXbl AccountInfoRegistry;
};

/* UE::Online */ }

#endif // WITH_GRDK
