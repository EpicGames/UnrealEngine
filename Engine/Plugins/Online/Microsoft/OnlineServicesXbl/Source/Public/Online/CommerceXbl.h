// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#if WITH_GRDK

#include "CoreMinimal.h"

#include "Online/CommerceCommon.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <XGameRuntime.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

#include "GDKHandle.h"

namespace UE::Online {

class FOnlineServicesXbl;

class ONLINESERVICESXBL_API FCommerceXbl : public FCommerceCommon
{
public:
	using Super = FCommerceCommon;

	FCommerceXbl(FOnlineServicesXbl& InServices);

	// TOnlineComponent
	virtual void Initialize() override;
	//void Tick(float DeltaSeconds) override;
	virtual void Shutdown() override;
	// ICommerce

	TOnlineAsyncOpHandle<FCommerceQueryOffers> QueryOffers(FCommerceQueryOffers::Params&& Params) override;
	TOnlineAsyncOpHandle<FCommerceQueryOffersById> QueryOffersById(FCommerceQueryOffersById::Params&& Params) override;

	TOnlineResult<FCommerceGetOffers> GetOffers(FCommerceGetOffers::Params&& Params) override;
	TOnlineResult<FCommerceGetOffersById> GetOffersById(FCommerceGetOffersById::Params&& Params) override;

	virtual TOnlineAsyncOpHandle<FCommerceShowStoreUI> ShowStoreUI(FCommerceShowStoreUI::Params&& Params) override;


	virtual TOnlineAsyncOpHandle<FCommerceQueryEntitlements> QueryEntitlements(FCommerceQueryEntitlements::Params&& Params) override;
	virtual TOnlineResult<FCommerceGetEntitlements> GetEntitlements(FCommerceGetEntitlements::Params&& Params) override;
	
	virtual TOnlineAsyncOpHandle<FCommerceRedeemEntitlement> RedeemEntitlement(FCommerceRedeemEntitlement::Params&& Params) override;

	virtual TOnlineAsyncOpHandle<FCommerceCheckout> Checkout(FCommerceCheckout::Params&& Params) override;

protected:

	bool ParseProducts(FAccountId LocalAccountId, XStoreProductQueryHandle ProductQueryHandle, TArray<FOfferId>* OfferIds);

	XStoreContextHandle GetStoreContextHandle(FGDKUserHandle GDKUser);
	void CleanupContextHandles();
	void HandleSuspend();

	TMap<FGDKUserHandle, XStoreContextHandle> StoreContexts;
	mutable FCriticalSection StoreContextsLock;

	bool bStoreContextHandlesInvalidated = false;

	TMap<FAccountId, TMap<FOfferId,FOffer>> CachedOffers;
	TMap<FAccountId, TMap<FEntitlementId, FEntitlement>> CachedEntitlements;

	mutable FCriticalSection CachedOffersLock;
	mutable FCriticalSection CachedEntitlementsLock;



};

/* UE::Online */
}
#endif // WITH_GRDK
