// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_GRDK
#include "Online/ConnectivityCommon.h"

namespace UE::Online {

class FOnlineServicesXbl;

class ONLINESERVICESXBL_API FConnectivityXbl : public FConnectivityCommon
{
public:
	using Super = FConnectivityCommon;

	using FConnectivityCommon::FConnectivityCommon;

	// TOnlineComponent
	virtual void PostInitialize() override;
	virtual void PreShutdown() override;

protected:

	void OnNetworkConnectionStatusChanged(ENetworkConnectionStatus LastConnectionState, ENetworkConnectionStatus ConnectionState);
};

/* UE::Online */}
#endif //WITH_GRDK
