// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#if WITH_GRDK

#include "Online/OnlineServicesCommon.h"
#include "Online/UserContextsManagerXbl.h"
#include "Online/EventsXbl.h"
#include "GDKHandle.h"
#include "GDKRuntimeModule.h"
#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <xsapi-c/types_c.h>
#include <xsapi-c/xbox_live_global_c.h>
#include <xsapi-c/xbox_live_context_c.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"


#define UE_XBL_ASYNC_BLOCK_KEY_NAME  TEXT("AsyncBlock")
#define UE_XBL_ACCOUNT_INFO_KEY_NAME  TEXT("AccountInfo")


namespace UE::Online 
{
	enum class EOnlineEnvironment : uint8
	{
		/** Dev environment */
		Development,
		/** Cert environment */
		Certification,
		/** Prod environment */
		Production,
		/** Not determined yet */
		Unknown

	};
	ENUM_CLASS_FLAGS(EOnlineEnvironment);

class ONLINESERVICESXBL_API FOnlineServicesXbl : public FOnlineServicesCommon
{
public:
	using Super = FOnlineServicesCommon;

	FOnlineServicesXbl(FName InInstanceName, FName InInstanceConfigName);
	virtual void RegisterComponents() override;
	virtual EOnlineServices GetServicesProvider() const override { return EOnlineServices::Xbox; }
	virtual void Initialize();
	virtual void PostInitialize();
	virtual void PreShutdown();

	virtual EOnlineEnvironment GetOnlineEnvironment() const;

	TSharedRef<FUserContextsManagerXbl> ContextManager;
	TSharedRef<FEventsXbl> EventLauncher;
private:	

	void InitializeXblByConfig();

	XblFunctionContext  ServiceCallRoutedHandlerContext = 0;
};

/* UE::Online */ }

#endif // WITH_GRDK
