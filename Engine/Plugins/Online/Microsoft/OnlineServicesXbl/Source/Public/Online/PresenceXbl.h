// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#if WITH_GRDK

#include "CoreMinimal.h"
#include "Online/PresenceCommon.h"

namespace UE::Online {

class FOnlineServicesXbl;
struct FOnlineStatusUpdate;
struct FTitleStatusUpdate;

class FPresenceXbl : public FPresenceCommon
{

public:

	FPresenceXbl(FOnlineServicesXbl& InServices);

	virtual void Initialize() override;
	virtual void PreShutdown() override;
	virtual void Tick(float DeltaSeconds) override;

	// IPresence
	virtual TOnlineAsyncOpHandle<FQueryPresence> QueryPresence(FQueryPresence::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FBatchQueryPresence> BatchQueryPresence(FBatchQueryPresence::Params&& Params) override;
	virtual TOnlineResult<FGetCachedPresence> GetCachedPresence(FGetCachedPresence::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FUpdatePresence> UpdatePresence(FUpdatePresence::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FPartialUpdatePresence> PartialUpdatePresence(FPartialUpdatePresence::Params&& Params) override;
	// End iPresence
private:
	uint32 TitleId = 0;
	FOnlineServicesXbl& Services;
	TMap<FAccountId, TMap<FAccountId, TSharedRef<FUserPresence>>> PresenceLists;
	TSharedRef<FUserPresence> FindOrCreatePresence(FAccountId LocalAccountId, FAccountId PresenceAccountId);
	void FindPresenceEntriesAndObservingLocalUsers(FAccountId PresenceAccountId, TArray<TSharedRef<FUserPresence>>& OutEntries, TArray<FAccountId>& OutObservers);

	FOnlineEventDelegateHandle TitleStatusUpdatedHandle;
	FOnlineEventDelegateHandle OnlineStatusUpdateHandle;
	void OnlineStatusUpdate(const FOnlineStatusUpdate& Update);
	void TitleStatusUpdate(const FTitleStatusUpdate& Update);

};

} // namespace UE::Online
#endif // WITH_GRDK
