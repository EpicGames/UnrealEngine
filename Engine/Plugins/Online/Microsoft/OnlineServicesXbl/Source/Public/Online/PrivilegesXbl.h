// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_GRDK

#include "CoreMinimal.h"
#include "HAL/CriticalSection.h"
#include "Online/PrivilegesCommon.h"
#include "GDKHandle.h"
#include "GDKRuntimeModule.h"
#include "GDKTaskQueueHelpers.h"

namespace UE::Online {

	class FOnlineServicesXbl;

	struct FPrivilegesXblConfig
	{
		bool bXBLGoldRequired = true;
		TArray<FString> IgnoredErrorCodes;
	};

	namespace Meta {

		BEGIN_ONLINE_STRUCT_META(FPrivilegesXblConfig)
			ONLINE_STRUCT_FIELD(FPrivilegesXblConfig, bXBLGoldRequired),
			ONLINE_STRUCT_FIELD(FPrivilegesXblConfig, IgnoredErrorCodes)
		END_ONLINE_STRUCT_META()
	/* Meta */	}



	class ONLINESERVICESXBL_API FPrivilegesXbl : public FPrivilegesCommon
	{
	public:
		using Super = FPrivilegesCommon;
		
		FPrivilegesXbl(FOnlineServicesXbl& InOwningSubsystem);
		virtual void Initialize() override;
		virtual void PreShutdown() override;

		// Begin IPrivileges

		TOnlineAsyncOpHandle<FQueryUserPrivilege> QueryUserPrivilege(FQueryUserPrivilege::Params&& Params);
		// End IPrivileges

	protected:

	};


/* UE::Online */ }

#endif // WITH_GRDK
