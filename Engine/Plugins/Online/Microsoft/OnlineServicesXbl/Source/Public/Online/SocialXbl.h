// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#if WITH_GRDK

#include "CoreMinimal.h"
#include "Online/SocialCommon.h"

namespace UE::Online {

class FOnlineServicesXbl;

class FSocialXbl : public FSocialCommon
{
public:

	FSocialXbl(FOnlineServicesXbl& InServices);

	// TOnlineComponent
	virtual void Initialize() override;
	virtual void PreShutdown() override;
	virtual void Tick(float DeltaSeconds) override;

	// Begin ISocial
	virtual TOnlineAsyncOpHandle<FQueryFriends> QueryFriends(FQueryFriends::Params&& Params) override;
	virtual TOnlineResult<FGetFriends> GetFriends(FGetFriends::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FSendFriendInvite> SendFriendInvite(FSendFriendInvite::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FQueryBlockedUsers> QueryBlockedUsers(FQueryBlockedUsers::Params&& Params) override;
	virtual TOnlineResult<FGetBlockedUsers> GetBlockedUsers(FGetBlockedUsers::Params&& Params) override;
	// End ISocial

private:

	FOnlineServicesXbl& Services;
	TMap<FAccountId, TMap<FAccountId, TSharedRef<FFriend>>> FriendsLists;
	TMap<FAccountId, TArray<FAccountId>> AvoidLists;

};

} // namespace UE::Online
#endif // WITH_GRDK
