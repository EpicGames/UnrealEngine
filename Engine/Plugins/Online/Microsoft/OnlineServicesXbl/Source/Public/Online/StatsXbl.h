// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_GRDK

#include "Online/StatsCommon.h"
#include "Online/OnlineComponent.h"
#include "Online/UserContextsManagerXbl.h"

namespace UE::Online {

enum class EXblStatMode : uint16
{
	/** 2013 stats are driven via events */
	Mode2013 = 2013,
	/** 2017 stats are title-managed */
	Mode2017 = 2017,
	/** Default mode to use if not specified */
	Default = Mode2013
};

struct FStatsXblConfig
{
	EXblStatMode StatMode = EXblStatMode::Default;
};

namespace Meta 
{
	BEGIN_ONLINE_STRUCT_META(FStatsXblConfig)
		ONLINE_STRUCT_FIELD(FStatsXblConfig, StatMode)
		END_ONLINE_STRUCT_META()
		/* Meta */
}

struct FOnlineStatusUpdate;
class ONLINESERVICESXBL_API FStatsXbl : public FStatsCommon
{
public:
	using Super = FStatsCommon;

	using FStatsCommon::FStatsCommon;

	virtual void Initialize() override;
	virtual void PreShutdown() override;

	// IStats
	virtual TOnlineAsyncOpHandle<FUpdateStats> UpdateStats(FUpdateStats::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FQueryStats> QueryStats(FQueryStats::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FBatchQueryStats> BatchQueryStats(FBatchQueryStats::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FTriggerEvent> TriggerEvent(FTriggerEvent::Params&& Params) override;
	// End IStats

protected:
	TOnlineAsyncOpRef<FBatchQueryStats> BatchQueryStatsImpl(FBatchQueryStats::Params&& InParams);

	FStatsXblConfig Config;
	FOnlineEventDelegateHandle StatUpdateHandle;
	void OnlineStatUpdate(const FOnlineStatUpdate& Update);
};

/* UE::Online */ }

#endif // WITH_GRDK
