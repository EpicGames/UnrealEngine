// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#if  WITH_GRDK
#include "CoreMinimal.h"
#include "Online/TitleFileCommon.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <xsapi-c/types_c.h>
#include <xsapi-c/title_storage_c.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

namespace UE::Online {

class FOnlineServicesXbl;


class FTitleFileXbl : public FTitleFileCommon
{
public:
	using Super = FTitleFileCommon;

	FTitleFileXbl(FOnlineServicesXbl& InOwningSubsystem);
	virtual ~FTitleFileXbl() = default;

	//// ITitleFile
	virtual TOnlineAsyncOpHandle<FTitleFileEnumerateFiles> EnumerateFiles(FTitleFileEnumerateFiles::Params&& Params) override;
	virtual TOnlineResult<FTitleFileGetEnumeratedFiles> GetEnumeratedFiles(FTitleFileGetEnumeratedFiles::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FTitleFileReadFile> ReadFile(FTitleFileReadFile::Params&& Params) override;

protected:

	TOptional<TArray<XblTitleStorageBlobMetadata>> EnumeratedFiles;

};

/* UE::Online */ }
#endif // WITH_GRDK
