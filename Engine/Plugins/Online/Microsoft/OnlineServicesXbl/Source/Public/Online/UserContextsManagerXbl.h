// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_GRDK


#include "CoreMinimal.h"
#include "GDKHandle.h"
#include "GDKRuntimeModule.h"
#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#include "Online/OnlineComponent.h"
#include "Online/Achievements.h"
#include "Templates/SharedPointerFwd.h"
THIRD_PARTY_INCLUDES_START
#include <xsapi-c/types_c.h>
#include <xsapi-c/xbox_live_global_c.h>
#include <xsapi-c/xbox_live_context_c.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

SETHANDLETYPETRAITS(XblContextHandle, XblContextDuplicateHandle, XblContextCloseHandle);

namespace UE::Online {

class FOnlineServicesXbl;
class FUserContextsManagerXbl;
static const XblFunctionContext INVALID_XBL_FUNCTION_CONTEXT = 0;

/**
 * Wrapper class representing a GDK user context.
 * Required as some delegates eg XblStatisticChangedHandler don't admit variable capture,
 * and we can only pass a single pointer in to the delegate, when we need both service instance and User Id.
 */
class UserContextWrapper : public TSharedFromThis<UserContextWrapper, ESPMode::ThreadSafe>
{
public:
	UserContextWrapper(TWeakPtr<FUserContextsManagerXbl, ESPMode::ThreadSafe> InManager, uint64 InXUID)
	{
		Manager = InManager;
		XUID = InXUID;
	}

	TWeakPtr<FUserContextsManagerXbl, ESPMode::ThreadSafe> Manager;
	uint64 XUID = 0;
};

typedef TSharedPtr<UserContextWrapper, ESPMode::ThreadSafe> UserContextWrapperPtr;
typedef TGDKHandle<XblContextHandle> FGDKContextHandle;

/** Struct for TitleStatusUpdate event */
struct FTitleStatusUpdate
{
	/* The affected account. */
	uint64 XUID;
	/* The active title. */
	uint64 TitleId;
	/* The new title state. */
	uint32 TitleState;
};

/** Struct for TitleStatusUpdate event */
struct FOnlineStatusUpdate
{
	/* The affected account. */
	uint64 XUID;
	/* Is the user online. */
	bool bOnline = false;
};

/** Struct for TitleStatusUpdate event */
struct FOnlineStatUpdate
{
	/* The affected account. */
	uint64 XUID;
	/* The affected stat. */
	FString StatisticName;
	/* The type of the affected stat. */
	FString StatisticType;
	/* The stats new value. */
	FString StatisticValue;

};

struct FAchievementUpdate
{
	/* The affected account. */
	uint64 XUID;
	/* The updated achievements. */
	TArray<FAchievementState> AchievementUpdates;
};

// Store single XboxGDKContext per user
struct FGDKContextInfo
{
	FGDKContextHandle Handle;
	XblFunctionContext AchievementChangedContext = INVALID_XBL_FUNCTION_CONTEXT;
	XblFunctionContext DevicePresenceChangedContext = INVALID_XBL_FUNCTION_CONTEXT;
	XblFunctionContext TitlePresenceChangedContext = INVALID_XBL_FUNCTION_CONTEXT;
	XblFunctionContext StatisticChangedContext = INVALID_XBL_FUNCTION_CONTEXT;
};

class  FUserContextsManagerXbl : public TSharedFromThis<FUserContextsManagerXbl, ESPMode::ThreadSafe>
{

public:
	FUserContextsManagerXbl(FOnlineServicesXbl* InService) { Service = InService; };
	virtual ~FUserContextsManagerXbl() = default;
	
	void CreateGDKContext(FGDKUserHandle GDKUser);
	void DeleteGDKContext(FGDKUserHandle GDKUser);
	void Shutdown();

	/** Returns the GDK context for the given user, or null if the input could not be found/was invalid. */
	FGDKContextHandle GetGDKContext(FPlatformUserId UserId);
	FGDKContextHandle GetGDKContext(uint64 XUID);
	FGDKContextHandle GetGDKContext(FGDKUserHandle GDKUser);

	virtual TOnlineEvent<void(const FTitleStatusUpdate&)> OnTitleStatusUpdate();
	virtual TOnlineEvent<void(const FOnlineStatusUpdate&)> OnOnlineStatusUpdate();
	virtual TOnlineEvent<void(const FOnlineStatUpdate&)> OnStatUpdate();
	virtual TOnlineEvent<void(const FAchievementUpdate&)> OnAchievementUpdate();

private:

	void UnsubscribeContextHandles(uint64 Xuid, FGDKContextInfo& ContextInfo) const;

	FOnlineServicesXbl* Service;
	TArray<UserContextWrapperPtr> UserContextWrappers;
	TMap<uint64, FGDKContextInfo> CachedGDKContexts;
	mutable FCriticalSection GDKContextsLock;

	TOnlineEventCallable<void(const FTitleStatusUpdate&)> OnTitleStatusUpdateEvent;
	TOnlineEventCallable<void(const FOnlineStatusUpdate&)> OnOnlineStatusUpdateEvent;
	TOnlineEventCallable<void(const FOnlineStatUpdate&)> OnStatUpdateEvent;
	TOnlineEventCallable<void(const FAchievementUpdate&)> OnAchievementUpdateEvent;
};

} // namespace UE::Online
#endif // WITH_GRDK

