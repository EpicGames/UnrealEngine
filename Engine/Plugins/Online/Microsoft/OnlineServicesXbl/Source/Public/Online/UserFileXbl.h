// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#if WITH_GRDK

#include "CoreMinimal.h"
#include "Online/UserFileCommon.h"

namespace UE::Online {

class FOnlineServicesXbl;

class FUserFileXbl : public FUserFileCommon
{
public:
	using Super = FUserFileCommon;

	FUserFileXbl(FOnlineServicesXbl& InOwningSubsystem);

	//// IUserFile
	virtual TOnlineAsyncOpHandle<FUserFileEnumerateFiles> EnumerateFiles(FUserFileEnumerateFiles::Params&& Params) override;
	virtual TOnlineResult<FUserFileGetEnumeratedFiles> GetEnumeratedFiles(FUserFileGetEnumeratedFiles::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FUserFileReadFile> ReadFile(FUserFileReadFile::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FUserFileWriteFile> WriteFile(FUserFileWriteFile::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FUserFileDeleteFile> DeleteFile(FUserFileDeleteFile::Params&& Params) override;

protected:

	TMap<FAccountId, TArray<FString>> UserToFiles;

};

/* UE::Online */ }
#endif // WITH_GRDK
