// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_GRDK

#include "Online/UserInfoCommon.h"
#include "Online/OnlineComponent.h"

struct XblUserProfile;
namespace UE::Online {

class ONLINESERVICESXBL_API FUserInfoXbl : public FUserInfoCommon
{
public:
	using Super = FUserInfoCommon;

	using FUserInfoCommon::FUserInfoCommon;

	// IUserInfo
	virtual TOnlineAsyncOpHandle<FQueryUserInfo> QueryUserInfo(FQueryUserInfo::Params&& Params) override;
	virtual TOnlineResult<FGetUserInfo> GetUserInfo(FGetUserInfo::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FQueryUserAvatar> QueryUserAvatar(FQueryUserAvatar::Params&& Params) override;
	virtual TOnlineResult<FGetUserAvatar> GetUserAvatar(FGetUserAvatar::Params&& Params) override;
	virtual TOnlineAsyncOpHandle<FShowUserProfile> ShowUserProfile(FShowUserProfile::Params&& Params) override;
	// End IUserInfo

protected:

	TMap<uint64, TSharedPtr<XblUserProfile>> UserProfileCache;

};

/* UE::Online */ }

#endif // WITH_GRDK
